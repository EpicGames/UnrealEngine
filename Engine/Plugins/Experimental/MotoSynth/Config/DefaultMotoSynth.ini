[CoreRedirects]

;[4.26]
+ClassRedirects=(OldName="/Script/Synthesis.SynthComponentMoto", NewName="/Script/MotoSynth.SynthComponentMoto")
+ClassRedirects=(OldName="/Script/Synthesis.MotoSynthPreset", NewName="/Script/MotoSynth.MotoSynthPreset")
+ClassRedirects=(OldName="/Script/Synthesis.MotoSynthPreset", NewName="/Script/MotoSynth.MotoSynthPreset")
+ClassRedirects=(OldName="/Script/Synthesis.MotoSynthSource", NewName="/Script/MotoSynth.MotoSynthSource")

