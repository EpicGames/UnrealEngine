{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "1",
	"FriendlyName" : "MotoSynth",
	"Description" : "An experimental granular vehicle engine. Intended to explore and demonstrate potential capabilities. Not supported.",
	"Category" : "Audio",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsExperimentalVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "MotoSynth",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "MotoSynthEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}