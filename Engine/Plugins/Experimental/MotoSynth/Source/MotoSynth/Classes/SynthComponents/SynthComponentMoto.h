// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SynthComponent.h"
#include "MotoSynthPreset.h"
#include "SynthComponentMoto.generated.h"

#define UE_API MOTOSYNTH_API


UCLASS(MinimalAPI, ClassGroup = Synth, meta = (BlueprintSpawnableComponent))
class USynthComponentMoto : public USynthComponent
{
	GENERATED_BODY()
public:

	UE_API USynthComponentMoto(const FObjectInitializer& ObjInitializer);
	UE_API virtual ~USynthComponentMoto();

	/* The moto synth preset to use for the moto synth component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MotoSynth")
	TObjectPtr<UMotoSynthPreset> MotoSynthPreset;

	/* Sets the starting RPM of the engine */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MotoSynth", meta = (ClampMin = "500.0", ClampMax = "20000.0", UIMin = "500.0", UIMax = "20000.0"))
	float RPM = 1000.0f;

	/** Sets the RPM of the granular engine directly. */
	UFUNCTION(BlueprintCallable, Category = "MotoSynth")
	UE_API void SetRPM(float InRPM, float InTimeSec);

	/** Sets a moto synth settings dynamically. */
	UFUNCTION(BlueprintCallable, Category = "MotoSynth")
	UE_API void SetSettings(const FMotoSynthRuntimeSettings& InSettings);

	/** Retrieves RPM range of the moto synth, taking into account the acceleration and deceleration sources. The min RPM is the largest of the min RPms of either and the max RPM is min of the max RPMs of either. */
	UFUNCTION(BlueprintCallable, Category = "MotoSynth")
	UE_API void GetRPMRange(float& OutMinRPM, float& OutMaxRPM);

	/** Returns if the moto synth is enabled. */
	UFUNCTION(BlueprintCallable, Category = "MotoSynth")
	UE_API bool IsEnabled() const;

	UE_API virtual ISoundGeneratorPtr CreateSoundGenerator(const FSoundGeneratorInitParams& InParams) override;

private:
	FMotoSynthRuntimeSettings* GetSettingsToUse();

	FCriticalSection SettingsCriticalSection;
	FVector2D RPMRange;
	ISoundGeneratorPtr MotoSynthEngine;
	FMotoSynthRuntimeSettings OverrideSettings;
	bool bSettingsOverridden = false;
};

#undef UE_API
