// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MotoSynth : ModuleRules
	{
		public MotoSynth(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"AudioExtensions",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AudioMixer",
					"SignalProcessing",
				}
			);
		}
	}
}