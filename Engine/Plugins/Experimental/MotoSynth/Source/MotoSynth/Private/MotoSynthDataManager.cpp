// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotoSynthDataManager.h"
#include "MotoSynthModule.h"

#include "HAL/IConsoleManager.h"
#include "MotoSynthSourceAsset.h"


// Log all loaded data in moto synth data manager
static FAutoConsoleCommand LogMotoSynthMemoryUsageCommand(
	TEXT("au.motosynth.logmemory"),
	TEXT("Logs all memory used by moto synth right now."),
	FConsoleCommandDelegate::CreateLambda([]() { FMotoSynthSourceDataManager::LogMemory(); })
);

static int32 MemoryLoggingCvar = 0;
FAutoConsoleVariableRef CVarMemoryLoggingCvar(
	TEXT("au.motosynth.enablememorylogging"),
	MemoryLoggingCvar,
	TEXT("Enables logging of memory usage whenever new sources are registered and unregistered.\n")
	TEXT("0: Disable, >0: Enable"),
	ECVF_Default);

static int32 MotoSynthBitCrushEnabledCvar = 0;
FAutoConsoleVariableRef CVarSetMotoSynthBitCrush(
	TEXT("au.motosynth.enablebitcrush"),
	MotoSynthBitCrushEnabledCvar,
	TEXT("Bit crushes moto synth source data to 8 bytes when registered to data manager.\n")
	TEXT("0: Disable, >0: Enable"),
	ECVF_Default);


FMotoSynthSourceDataManager& FMotoSynthSourceDataManager::Get()
{
	static FMotoSynthSourceDataManager* MotoSynthDataManager = nullptr;
	if (!MotoSynthDataManager)
	{
		MotoSynthDataManager = new FMotoSynthSourceDataManager();
	}
	return *MotoSynthDataManager;
}

void FMotoSynthSourceDataManager::LogMemory()
{
	FMotoSynthSourceDataManager& MotoSynthDataManager = FMotoSynthSourceDataManager::Get();
	MotoSynthDataManager.LogMemoryUsage();
}

FMotoSynthSourceDataManager::FMotoSynthSourceDataManager()
{

}

FMotoSynthSourceDataManager::~FMotoSynthSourceDataManager()
{

}

void FMotoSynthSourceDataManager::RegisterData(uint32 InSourceID, const FName& InSourceName, TArray<int16>&& InSourceDataPCM, int32 InSourceSampleRate, TArray<FGrainTableEntry>&& InGrainTable, const FRichCurve& InRPMCurve, bool bConvertTo8Bit)
{
	LLM_SCOPE(ELLMTag::AudioSynthesis);

	FScopeLock ScopeLock(&DataCriticalSection);

	if (SourceDataTable.Contains(InSourceID))
	{
		UE_LOGF(LogMotoSynth, Error, "Moto synth source data already registered for source ID %d", InSourceID);
		return;
	}

	MotoSynthDataPtr NewData = MotoSynthDataPtr(new FMotoSynthSourceData);

	if (MotoSynthBitCrushEnabledCvar == 1 || bConvertTo8Bit)
	{
		// if 8-bit mode is enabled, convert the PCM data to 8-bit PCM data
		TArray<int16> SourceData = MoveTemp(InSourceDataPCM);
		NewData->AudioSourceBitCrushed.AddUninitialized(SourceData.Num());

		int16* SourceDataPtr = SourceData.GetData();
		uint8* SourceDataBitCrushedPtr = NewData->AudioSourceBitCrushed.GetData();
		for (int32 SampleIndex = 0; SampleIndex < SourceData.Num(); SampleIndex++)
		{
			// take -1.0 to 1.0 range to 0.0 to 1.0
			float PolarSampleValue = 0.5f * ((float)SourceDataPtr[SampleIndex] / 32767.0f + 1.0f);
			
			// Scale the polar sample to 8-bit data
			uint8 Sample8Bit = (uint8)(255.0f * PolarSampleValue);
			SourceDataBitCrushedPtr[SampleIndex] = Sample8Bit;
		}
	}
	else
	{
		NewData->AudioSource = MoveTemp(InSourceDataPCM);
	}

	NewData->SourceSampleRate = InSourceSampleRate;
	NewData->GrainTable = MoveTemp(InGrainTable);
	NewData->RPMCurve = InRPMCurve;
	NewData->SourceName = InSourceName;

	SourceDataTable.Add(InSourceID, NewData);

	if (MemoryLoggingCvar == 1)
	{
		int32 SourceDataSizeBytes = InSourceDataPCM.Num() * sizeof(int16);
		int32 GrainTableDataSizeBytes = InGrainTable.Num() * sizeof(FGrainTableEntry);
		UE_LOGF(LogMotoSynth, Display, "Registering New Moto Synth Source (Name: %ls, Id: %d), Source Size %d MB, Grain Table Size %d MB", *InSourceName.ToString(), InSourceID, SourceDataSizeBytes / (1024 * 1024), GrainTableDataSizeBytes / (1024 * 1024));
		LogMemoryUsage();
	}
}

void FMotoSynthSourceDataManager::UnRegisterData(uint32 InSourceID)
{
	LLM_SCOPE(ELLMTag::AudioSynthesis);

	FScopeLock ScopeLock(&DataCriticalSection);

	// Find the entry
	int32 NumRemoved = SourceDataTable.Remove(InSourceID);
	if (NumRemoved == 0)
	{
		UE_LOGF(LogMotoSynth, Error, "No entry in moto synth source data entry for moto synth source ID %d", InSourceID);
	}

	if (NumRemoved == 1 && MemoryLoggingCvar == 1)
	{
		UE_LOGF(LogMotoSynth, Display, "Unregistering Moto Synth Source (Id: %d)", InSourceID);
		LogMemoryUsage();
	}
}

MotoSynthDataPtr FMotoSynthSourceDataManager::GetMotoSynthData(uint32 InSourceID)
{
	FScopeLock ScopeLock(&DataCriticalSection);

	MotoSynthDataPtr* SourceData = SourceDataTable.Find(InSourceID);
	if (SourceData)
	{
		return *SourceData;
	}
	else
	{
		UE_LOGF(LogMotoSynth, Error, "Unable to get moto synth data view for source ID %d", InSourceID);
		return nullptr;
	}
}

void FMotoSynthSourceDataManager::LogMemoryUsage()
{	
	int32 NumSources = SourceDataTable.Num();
	int32 NumBytesSource = 0;
	int32 NumBytesGrainTable = 0;
	for (auto& Entry : SourceDataTable)
	{
		MotoSynthDataPtr& MotoSynthData = Entry.Value;
		if (MotoSynthData->AudioSourceBitCrushed.Num() > 0)
		{
			NumBytesSource += MotoSynthData->AudioSourceBitCrushed.Num() * sizeof(uint8);
		}
		else
		{
			NumBytesSource += MotoSynthData->AudioSource.Num() * sizeof(int16);
		}
		NumBytesGrainTable += MotoSynthData->GrainTable.Num() * sizeof(FGrainTableEntry);
	}
	float NumMBSource = (float)NumBytesSource / (1024.0f * 1024.0f);
	float NumMBGrainTable = (float)NumBytesGrainTable / (1024.0f * 1024.0f);

	UE_LOGF(LogMotoSynth, Display, "MotoSynthSource Data: %d Sources, %.2f MB source, %.2f MB grain table (%.2f MB total)", NumSources, NumMBSource, NumMBGrainTable, NumMBSource + NumMBGrainTable);
}



