// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RichCurve.h"

struct FGrainTableEntry;

struct FMotoSynthSourceData
{
	// The raw audio source used for the data
	TArray<int16> AudioSource;

	// Bit-crushed audio source data (reduced bit-size)
	TArray<uint8> AudioSourceBitCrushed;

	// The sample rate of the source file
	int32 SourceSampleRate = 0;

	// The RPM pitch curve used for the source
	FRichCurve RPMCurve;

	// The name of the source
	FName SourceName;

	// The grain table derived during editor time
	TArray<FGrainTableEntry> GrainTable;
};

typedef TSharedPtr<FMotoSynthSourceData, ESPMode::ThreadSafe> MotoSynthDataPtr;

class FMotoSynthSourceDataManager
{
public:
	static FMotoSynthSourceDataManager& Get();
	static void LogMemory();

	// Registers the moto synth source data 
	void RegisterData(uint32 InSourceID, const FName& InSourceName, TArray<int16>&& InSourceDataPCM, int32 InSourceSampleRate, TArray<FGrainTableEntry>&& InGrainTable, const FRichCurve& InRPMCurve, bool bConvertTo8Bit);
	void UnRegisterData(uint32 InSourceID);

	// Retrieves data view for the given SourceID if the source data exists. 
	// Returns true if the data existed for the SourceID, false if it did not.
	MotoSynthDataPtr GetMotoSynthData(uint32 SourceID);

	// Logs the memory usage of this data manager
	void LogMemoryUsage();

private:
	FMotoSynthSourceDataManager();
	~FMotoSynthSourceDataManager();

	TMap<uint32, MotoSynthDataPtr> SourceDataTable;

	FCriticalSection DataCriticalSection;

};
