// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotoSynthEngine.h"
#include "DSP/FloatArrayMath.h"
#include "HAL/IConsoleManager.h"
#include "MotoSynthPreset.h"
#include "MotoSynthSourceAsset.h"

static int32 MotosynthDisabledCVar = 0;
FAutoConsoleVariableRef CVarDisableMotoSynth(
	TEXT("au.DisableMotoSynth"),
	MotosynthDisabledCVar,
	TEXT("Disables the moto synth.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

// Retrieves the moto synth asset manager
FMotoSynthAssetManager& FMotoSynthAssetManager::Get()
{
	static TUniquePtr<FMotoSynthAssetManager> MotoSynthAssetManager;

	if (!MotoSynthAssetManager.IsValid())
	{
		MotoSynthAssetManager = TUniquePtr<FMotoSynthAssetManager>(new FMotoSynthAssetManager);
	}
	return *MotoSynthAssetManager.Get();
}

FMotoSynthAssetManager::FMotoSynthAssetManager()
{
}

FMotoSynthAssetManager::~FMotoSynthAssetManager()
{
}

bool FMotoSynthEngine::IsMotoSynthEngineEnabled()
{
	return MotosynthDisabledCVar == 0;
}

FMotoSynthEngine::FMotoSynthEngine()
{
}

FMotoSynthEngine::~FMotoSynthEngine()
{
}

void FMotoSynthEngine::Init(int32 InSampleRate)
{
	if (!IsMotoSynthEngineEnabled())
	{
		return;
	}

	RendererSampleRate = InSampleRate;
	CurrentRPM = 0.0f;

	SynthOsc.Init((float)InSampleRate);
	SynthOsc.SetType(Audio::EOsc::Saw);

	SynthOsc.SetGain(0.5f);
	SynthOsc.SetFrequency(100.0f);
	SynthOsc.Update();

	SynthOsc.Start();

	SynthEnv.Init(InSampleRate);

	NoiseEnv.Init(InSampleRate);
	NoiseFilter.Init((float)InSampleRate, 1, Audio::EBiquadFilter::Lowpass);

	GrainCrossfadeSamples = 10;
	NumGrainTableEntriesPerGrain = 3;
	GrainTableRandomOffsetForConstantRPMs = 20;
	GrainCrossfadeSamplesForConstantRPMs = 20;

	SynthOctaveShift = 0;
	SynthFilter.Init((float)InSampleRate, 1);

	DelayStereo.Init((float)InSampleRate, 2);
	DelayStereo.SetDelayTimeMsec(25);
	DelayStereo.SetFeedback(0.37f);
	DelayStereo.SetWetLevel(0.68f);
	DelayStereo.SetDryLevel(0.8f);
	DelayStereo.SetDelayRatio(0.43f);
	DelayStereo.SetMode(Audio::EStereoDelayMode::PingPong);
	DelayStereo.SetFilterEnabled(true);
	DelayStereo.SetFilterSettings(Audio::EBiquadFilter::Lowpass, 4000.0f, 0.5f);

	GrainEnvelope.GenerateEnvelope(Audio::EGrainEnvelopeType::Hanning, 512);

	constexpr int32 GrainPoolSize = 10;
	GrainPool.Init(FMotoSynthGrainRuntime(), GrainPoolSize);
	for (int32 i = 0; i < GrainPoolSize; ++i)
	{
		FreeGrains.Add(i);
	}

	ActiveGrains.Reset();

	bRPMWasSet = false;
}

void FMotoSynthEngine::Reset()
{
	SynthCommand([this]()
	{
		Init(RendererSampleRate);
	});
}

void FMotoSynthEngine::SetSourceData(uint32 InAccelerationSourceID, uint32 InDecelerationSourceID)
{
	FMotoSynthSourceDataManager& DataManager = FMotoSynthSourceDataManager::Get();
	MotoSynthDataPtr InAccelerationSourceData = DataManager.GetMotoSynthData(InAccelerationSourceID);
	MotoSynthDataPtr InDecelerationSourceData = DataManager.GetMotoSynthData(InDecelerationSourceID);

	if (InAccelerationSourceData.IsValid() && InDecelerationSourceData.IsValid())
	{
		FVector2f AccelRPMRange;
		InAccelerationSourceData->RPMCurve.GetValueRange(AccelRPMRange.X, AccelRPMRange.Y);

		FVector2f DecelRPMRange;
		InDecelerationSourceData->RPMCurve.GetValueRange(DecelRPMRange.X, DecelRPMRange.Y);

		FVector2f NewRPMRange = { FMath::Max(AccelRPMRange.X, DecelRPMRange.X), FMath::Min(AccelRPMRange.Y, DecelRPMRange.Y) };
		RPMRange = NewRPMRange;

		SynthCommand([this, InAccelerationSourceData, InDecelerationSourceData, NewRPMRange]() mutable
		{
			CurrentAccelerationSourceDataIndex = 0;
			CurrentDecelerationSourceDataIndex = 0;
			AccelerationSourceData = InAccelerationSourceData;
			DecelerationSourceData = InDecelerationSourceData;
			RPMRange_RendererCallback = NewRPMRange;
		});
	}
}

void FMotoSynthEngine::GetRPMRange(FVector2D& OutRPMRange)
{
	OutRPMRange = FVector2D(RPMRange);
}

void FMotoSynthEngine::SetSettings(const FMotoSynthRuntimeSettings& InSettings)
{
	SynthCommand([this, InSettings]()
	{
		bSynthToneEnabled = InSettings.bSynthToneEnabled;
		bSynthToneEnvelopeEnabled = InSettings.bSynthToneEnvelopeEnabled;
		SynthOctaveShift = InSettings.SynthOctaveShift;

		SynthToneVolumeRange = { FMath::Clamp<float>(InSettings.SynthToneVolumeRange.X, 0.0f, 4.0f), FMath::Clamp<float>(InSettings.SynthToneVolumeRange.Y, 0.0f, 4.0f) };
		SynthToneFilterFrequencyRange = { FMath::Clamp<float>(InSettings.SynthToneFilterFrequencyRange.X, 20.0f, 10000.0f), FMath::Clamp<float>(InSettings.SynthToneFilterFrequencyRange.Y, 20.0f, 10000.0f) };;
		SynthToneAttackTimeMsecRange = { FMath::Max<float>(InSettings.SynthToneAttackTimeMsecRange.X, 0.0f), FMath::Max<float>(InSettings.SynthToneAttackTimeMsecRange.Y, 0.0f) };
		SynthToneDecayTimeMsecRange = { FMath::Max<float>(InSettings.SynthToneDecayTimeMsecRange.X, 0.0f), FMath::Max<float>(InSettings.SynthToneDecayTimeMsecRange.Y, 0.0f) };
		SynthToneAttackCurveRange = { FMath::Max<float>(InSettings.SynthToneAttackCurveRange.X, 0.0f), FMath::Max<float>(InSettings.SynthToneAttackCurveRange.Y, 0.0f) };
		SynthToneDecayCurveRange = { FMath::Max<float>(InSettings.SynthToneDecayCurveRange.X, 0.0f), FMath::Max<float>(InSettings.SynthToneDecayCurveRange.Y, 0.0f) };

		bGranularEngineEnabled = InSettings.bGranularEngineEnabled;
		TargetGranularEngineVolume = InSettings.GranularEngineVolume;
		GrainCrossfadeSamples = InSettings.NumSamplesToCrossfadeBetweenGrains;
		NumGrainTableEntriesPerGrain = InSettings.NumGrainTableEntriesPerGrain;
		GrainTableRandomOffsetForConstantRPMs = InSettings.GrainTableRandomOffsetForConstantRPMs;
		GrainCrossfadeSamplesForConstantRPMs = InSettings.GrainCrossfadeSamplesForConstantRPMs;
		bStereoWidenerEnabled = InSettings.bStereoWidenerEnabled;
		PitchScale = InSettings.GranularEnginePitchScale;

		bNoiseEnabled = InSettings.bNoiseEnabled;
		bNoiseEnvelopeEnabled = InSettings.bNoiseEnvelopeEnabled;

		NoiseVolumeRange = { FMath::Clamp<float>(InSettings.NoiseVolumeRange.X, 0.0f, 4.0f), FMath::Clamp<float>(InSettings.NoiseVolumeRange.Y, 0.0f, 4.0f) };
		NoiseLPFRange = { FMath::Clamp<float>(InSettings.NoiseLPFRange.X, 20.0f, 20000.0f), FMath::Clamp<float>(InSettings.NoiseLPFRange.Y, 20.0f, 20000.0f) };
		NoiseAttackTimeMsecRange = { FMath::Max<float>(InSettings.NoiseAttackTimeMsecRange.X, 0.0f), FMath::Max<float>(InSettings.NoiseAttackTimeMsecRange.Y, 0.0f) };
		NoiseDecayTimeMsecRange = { FMath::Max<float>(InSettings.NoiseDecayTimeMsecRange.X, 0.0f), FMath::Max<float>(InSettings.NoiseDecayTimeMsecRange.Y, 0.0f) };
		NoiseAttackCurveRange = { FMath::Max<float>(InSettings.NoiseAttackCurveRange.X, 0.0f), FMath::Max<float>(InSettings.NoiseAttackCurveRange.Y, 0.0f) };
		NoiseDecayCurveRange = { FMath::Max<float>(InSettings.NoiseDecayCurveRange.X, 0.0f), FMath::Max<float>(InSettings.NoiseDecayCurveRange.Y, 0.0f) };

		DelayStereo.SetDelayTimeMsec(InSettings.StereoDelayMsec);
		DelayStereo.SetFeedback(InSettings.StereoFeedback);
		DelayStereo.SetWetLevel(InSettings.StereoWidenerWetlevel);
		DelayStereo.SetDryLevel(InSettings.StereoWidenerDryLevel);
		DelayStereo.SetDelayRatio(InSettings.StereoWidenerDelayRatio);
		DelayStereo.SetFilterEnabled(InSettings.bStereoWidenerFilterEnabled);
		DelayStereo.SetFilterSettings(Audio::EBiquadFilter::Lowpass, InSettings.StereoWidenerFilterFrequency, InSettings.StereoWidenerFilterQ);
	});
}

void FMotoSynthEngine::SetRPM(float InRPM, float InTimeSec)
{
	SynthCommand([this, InRPM, InTimeSec]()
	{
		bRPMWasSet = true;
		TargetRPM = InRPM;
		CurrentRPMTime = 0.0f;
		RPMFadeTime = InTimeSec;
		StartingRPM = CurrentRPM;

		if (CurrentRPM == 0.0f)
		{
			StartingRPM = TargetRPM;
			CurrentRPM = TargetRPM;
			PreviousRPM = TargetRPM - 1;
			CurrentRPMSlope = 0.0f;
			PreviousRPMSlope = 0.0f;
			bWasAccelerating = true;
		}
	});
}

void FMotoSynthEngine::SetPitchScale(float InPitchScale)
{
	SynthCommand([this, InPitchScale]()
	{
		PitchScale = FMath::Clamp(InPitchScale, 0.01f, 10.0f);
	});
}

bool FMotoSynthEngine::NeedsSpawnGrain()
{
	if (ActiveGrains.Num() > 0)
	{
		if (ActiveGrains.Num() == 1)
		{
			int32 GrainIndex = ActiveGrains[0];
			if (GrainPool[GrainIndex].IsNearingEnd())
			{
				return true;
			}
		}
		// No grains needed spawning
		return false;
	}
	// No active grains so return true
	return true;
}

void FMotoSynthEngine::SpawnGrain(int32& StartingIndex, const MotoSynthDataPtr& SynthData)
{
	if (FreeGrains.Num() && CurrentRPM > 0.0f && SynthData)
	{
		int32 GrainIndex = FreeGrains.Pop(EAllowShrinking::No);
		ActiveGrains.Push(GrainIndex);

		FMotoSynthGrainRuntime& NewGrain = GrainPool[GrainIndex];
		
		// Start a bit to the left
		int32 GrainTableStart = FMath::Max(StartingIndex - 1, 0);
		for (int32 GrainTableIndex = GrainTableStart; GrainTableIndex < SynthData->GrainTable.Num(); ++GrainTableIndex)
		{
			const FGrainTableEntry* Entry = &SynthData->GrainTable[GrainTableIndex];
			if ((CurrentRPMSlope >= 0.0f && Entry->RPM >= CurrentRPM) || (CurrentRPMSlope < 0.0f && Entry->RPM < CurrentRPM))
			{
				// If the grain we're picking is the exact same one, lets randomly pick a grain around here
				int32 NumGrainEntries = NumGrainTableEntriesPerGrain;
				int32 NewGrainCrossfadeSamples = GrainCrossfadeSamples;
				if (StartingIndex == GrainTableIndex)
				{
					NewGrainCrossfadeSamples = GrainCrossfadeSamplesForConstantRPMs;

					GrainTableIndex += FMath::RandRange(-GrainTableRandomOffsetForConstantRPMs, GrainTableRandomOffsetForConstantRPMs);
					GrainTableIndex = FMath::Clamp(GrainTableIndex, 0, SynthData->GrainTable.Num());
				}
				else 
				{
					// Update the starting index that was passed in to optimize grain-table look up for future spawns
					StartingIndex = GrainTableIndex;
				}

				// compute the grain duration based on the NumGrainEntriesPerGrain
				// we walk the grain table and add grain table durations together to reach a final duration		
				int32 NextGrainTableIndex = GrainTableIndex + NumGrainEntries + 1;
				int32 NextGrainTableIndexClamped = FMath::Clamp(NextGrainTableIndex, 0, SynthData->GrainTable.Num() - 1);
				int32 GrainTableIndexClamped = FMath::Clamp(GrainTableIndex, 0, SynthData->GrainTable.Num() - 1);
				int32 GrainDuration = SynthData->GrainTable[NextGrainTableIndexClamped].SampleIndex - SynthData->GrainTable[GrainTableIndexClamped].SampleIndex;

				// Get the RPM value for the very next grain after this grain duration to be the "ending rpm"
				// This allows us to pitch-scale the grain more closely to the grain's RPM contour through it's lifetime
				int32 EndingRPM = SynthData->GrainTable[NextGrainTableIndexClamped].RPM;

				int32 StartIndex = FMath::Max(0, Entry->SampleIndex - NewGrainCrossfadeSamples);

				FGrainInitParams GrainInitParams;
				GrainInitParams.GrainEnvelope = &GrainEnvelope;
				GrainInitParams.NumSamplesCrossfade = NewGrainCrossfadeSamples;
				GrainInitParams.GrainStartRPM = Entry->RPM;
				GrainInitParams.GrainEndRPM = EndingRPM;
				GrainInitParams.StartingRPM = CurrentRPM;
				GrainInitParams.EnginePitchScale = PitchScale * ((float)SynthData->SourceSampleRate / RendererSampleRate);

				if (SynthData->AudioSourceBitCrushed.Num() > 0)
				{
					int32 EndIndex = FMath::Min(Entry->SampleIndex + GrainDuration + NewGrainCrossfadeSamples, SynthData->AudioSourceBitCrushed.Num());
					GrainInitParams.GrainView = MakeArrayView(&SynthData->AudioSourceBitCrushed[StartIndex], EndIndex - StartIndex);
					GrainInitParams.NumBytesPerSample = 1;
				}
				else
				{
					int32 EndIndex = FMath::Min(Entry->SampleIndex + GrainDuration + NewGrainCrossfadeSamples, SynthData->AudioSource.Num());
					uint8* AudioSourceData = (uint8*)&SynthData->AudioSource[StartIndex];
					GrainInitParams.GrainView = MakeArrayView(AudioSourceData, 2 * (EndIndex - StartIndex));
					GrainInitParams.NumBytesPerSample = 2;
				}

				NewGrain.Init(GrainInitParams);
				NewGrain.SetRPM(CurrentRPM);

				break;
			}
		}
	}
}

void FMotoSynthEngine::GenerateGranularEngine(float* OutAudio, int32 NumSamples)
{
	if (!bRPMWasSet)
	{
		return;
	}

	// If we're generating a synth tone prepare the scratch buffer
	if (bSynthToneEnabled || bNoiseEnabled)
	{
		SynthBuffer.Reset();
		SynthBuffer.AddUninitialized(NumSamples);
	}

	// we lerp through the frame lerp to accurately account for RPM changes and accel or decel
	float RPMDelta = 0.0f;
	if (!FMath::IsNearlyEqual(CurrentRPM, TargetRPM) || !RPMFadeTime)
	{
		// In this callback, we will lerp to a target RPM. We will always lerp to the target, even if it's in one callback.
		float ThisCallbackTargetRPM = TargetRPM;

		// If we've been given a non-zero fade time, then we will likely lerp through multiple callbacks to get to our target
		// So we need to figure out what percentage of our target RPM value we need to lerp to
		if (RPMFadeTime > 0.0f)
		{
			// Update the RPM time at the callback block rate. Next callback we'll progress further through the lerp.
			CurrentRPMTime += (float)NumSamples / RendererSampleRate;

			// Track how far we've progressed through the fade time
			float FadeFraction = FMath::Clamp(CurrentRPMTime / RPMFadeTime, 0.0f, 1.0f);

			// Compute the fraction of how far we are with the overrall target RPM and which RPM we started at.
			ThisCallbackTargetRPM = StartingRPM + FadeFraction * (TargetRPM - StartingRPM);
		}

		// This is the amount of RPMs to increment per sample to get accurate grain management
		RPMDelta = (ThisCallbackTargetRPM - CurrentRPM) / NumSamples;

		if (bSynthToneEnabled)
		{
			float SynthFilterFreq = FMath::GetMappedRangeValueClamped(RPMRange, SynthToneFilterFrequencyRange, CurrentRPM);
			SynthFilter.SetFrequency(SynthFilterFreq);
			SynthFilter.Update();

			float AttackCurve = FMath::GetMappedRangeValueClamped(RPMRange, SynthToneAttackCurveRange, CurrentRPM);
			SynthEnv.SetAttackCurveFactor(AttackCurve);

			float AttackTime = 0.001f * FMath::GetMappedRangeValueClamped(RPMRange, SynthToneAttackTimeMsecRange, CurrentRPM);
			SynthEnv.SetAttackTimeSeconds(AttackTime);

			float DecayCurve = FMath::GetMappedRangeValueClamped(RPMRange, SynthToneDecayCurveRange, CurrentRPM);
			SynthEnv.SetDecayCurveFactor(DecayCurve);

			float DecayTime = 0.001f * FMath::GetMappedRangeValueClamped(RPMRange, SynthToneDecayTimeMsecRange, CurrentRPM);
			SynthEnv.SetDecayTimeSeconds(DecayTime);
		}

		// If RPM changes, adjust the noise filter frequency if it's enabled
		if (bNoiseEnabled)
		{
			float NoiseLPF = FMath::GetMappedRangeValueClamped(RPMRange, NoiseLPFRange, CurrentRPM);
			NoiseFilter.SetFrequency(NoiseLPF);

			float AttackCurve = FMath::GetMappedRangeValueClamped(RPMRange, NoiseAttackCurveRange, CurrentRPM);
			NoiseEnv.SetAttackCurveFactor(AttackCurve);

			float AttackTime = 0.001f * FMath::GetMappedRangeValueClamped(RPMRange, NoiseAttackTimeMsecRange, CurrentRPM);
			NoiseEnv.SetAttackTimeSeconds(AttackTime);

			float DecayCurve = FMath::GetMappedRangeValueClamped(RPMRange, NoiseDecayCurveRange, CurrentRPM);
			NoiseEnv.SetDecayCurveFactor(DecayCurve);

			float DecayTime = 0.001f * FMath::GetMappedRangeValueClamped(RPMRange, NoiseDecayTimeMsecRange, CurrentRPM);
			NoiseEnv.SetDecayTimeSeconds(DecayTime);
		}
	}

	SynthEnvBuffer.Reset();
	NoiseEnvBuffer.Reset();

	for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
	{
		{
			if (NeedsSpawnGrain())
			{
				CurrentRPMSlope = CurrentRPM - PreviousRPM;
				bool bUnchanged = FMath::IsNearlyZero(CurrentRPMSlope, 0.001f);
				if (bUnchanged)
				{
					// If we haven't changed our RPM much, lets 
					if (bWasAccelerating)
					{
						CurrentDecelerationSourceDataIndex = 0;
						SpawnGrain(CurrentAccelerationSourceDataIndex, AccelerationSourceData);
					}
					else
					{
						CurrentAccelerationSourceDataIndex = 0;
						SpawnGrain(CurrentDecelerationSourceDataIndex, DecelerationSourceData);
					}
				}
				else if (CurrentRPMSlope > 0.0f)
				{
					bWasAccelerating = true;
					CurrentDecelerationSourceDataIndex = 0;
					SpawnGrain(CurrentAccelerationSourceDataIndex, AccelerationSourceData);
				}
				else
				{
					bWasAccelerating = false;
					CurrentAccelerationSourceDataIndex = 0;
					SpawnGrain(CurrentDecelerationSourceDataIndex, DecelerationSourceData);
				}

				// On every grain spawn, trigger an attack
				SynthEnv.Attack();
				NoiseEnv.Attack();
			}

			PreviousRPMSlope = CurrentRPMSlope;

			// Now render the grain sample data
			for (int32 ActiveGrainIndex = ActiveGrains.Num() - 1; ActiveGrainIndex >= 0; --ActiveGrainIndex)
			{
				int32 GrainIndex = ActiveGrains[ActiveGrainIndex];
				FMotoSynthGrainRuntime& Grain = GrainPool[GrainIndex];
				Grain.SetRPM(CurrentRPM);

				if (bGranularEngineEnabled)
				{
					OutAudio[SampleIndex] += Grain.GenerateSample();
				}
				else
				{
					// Still need to generate the grain since envelope state is dependent on grain state
					// TODO: if this is used at runtime, implement a virtual GenerateSample to update state but not do any audio work 
					// Currently the expectation is that this mode is only an editor-only mode (i.e. no pure synth engines shipped)
					Grain.GenerateSample();
				}

				if (Grain.IsDone())
				{
					ActiveGrains.RemoveAtSwap(ActiveGrainIndex, EAllowShrinking::No);
					FreeGrains.Push(GrainIndex);
				}
			}

			// Render the envelope data
			if (bSynthToneEnvelopeEnabled)
			{
				float EnvOut = 0.0f;
				SynthEnv.GetNextEnvelopeOut(EnvOut);
				SynthEnvBuffer.Add(EnvOut);
			}

			if (bNoiseEnvelopeEnabled)
			{
				float EnvOut = 0.0f;
				NoiseEnv.GetNextEnvelopeOut(EnvOut);
				NoiseEnvBuffer.Add(EnvOut);
			}
		}

		// We need to generate the synth tone with the exact sample RPM frequencies that the grains are
		if (bSynthToneEnabled)
		{
			if ((SynthPitchUpdateSampleIndex & SynthPitchUpdateDeltaSamples) == 0)
			{
				float CurrentFrequency = CurrentRPM / 60.0f;
				CurrentFrequency *= Audio::GetFrequencyMultiplier(12.0f * SynthOctaveShift);
				SynthOsc.SetFrequency(CurrentFrequency);
				SynthOsc.Update();
			}

			SynthBuffer[SampleIndex] = SynthOsc.Generate();
		}

		PreviousRPM = CurrentRPM;
		CurrentRPM += RPMDelta;
	}

	TArrayView<float> OutAudioView(OutAudio, NumSamples);

	if (!FMath::IsNearlyEqual(TargetGranularEngineVolume, GranularEngineVolume))
	{
		Audio::ArrayFade(OutAudioView, GranularEngineVolume, TargetGranularEngineVolume);
		GranularEngineVolume = TargetGranularEngineVolume;
	}
	else if (!FMath::IsNearlyEqual(GranularEngineVolume, 1.0f))
	{
		Audio::ArrayMultiplyByConstantInPlace(OutAudioView, GranularEngineVolume);
	}

	// Apply the filter and mix the audio into the output buffer
	if (bSynthToneEnabled)
	{
		float* SynthBufferPtr = SynthBuffer.GetData();
		SynthFilter.ProcessAudio(SynthBufferPtr, NumSamples, SynthBufferPtr);

		float SynthToneVolume = FMath::GetMappedRangeValueClamped(RPMRange, SynthToneVolumeRange, CurrentRPM);

		if (bSynthToneEnvelopeEnabled)
		{
			for (int32 FrameIndex = 0; FrameIndex < NumSamples; ++FrameIndex)
			{
				OutAudio[FrameIndex] += SynthToneVolume * SynthEnvBuffer[FrameIndex] * SynthBufferPtr[FrameIndex];
			}
		}
		else
		{
			for (int32 FrameIndex = 0; FrameIndex < NumSamples; ++FrameIndex)
			{
				OutAudio[FrameIndex] += SynthToneVolume * SynthBufferPtr[FrameIndex];
			}
		}
	}

	if (bNoiseEnabled)
	{
		float* SynthBufferPtr = SynthBuffer.GetData();
		for (int32 FrameIndex = 0; FrameIndex < NumSamples; ++FrameIndex)
		{
			SynthBufferPtr[FrameIndex] = NoiseGen.Generate();
		}

		float NoiseVolume = FMath::GetMappedRangeValueClamped(RPMRange, NoiseVolumeRange, CurrentRPM);

		// Filter it
		NoiseFilter.ProcessAudio(SynthBufferPtr, NumSamples, SynthBufferPtr);

		if (bNoiseEnvelopeEnabled)
		{
			for (int32 FrameIndex = 0; FrameIndex < NumSamples; ++FrameIndex)
			{
				OutAudio[FrameIndex] += NoiseVolume * NoiseEnvBuffer[FrameIndex] * SynthBufferPtr[FrameIndex];
			}
		}
		else
		{
			for (int32 FrameIndex = 0; FrameIndex < NumSamples; ++FrameIndex)
			{
				OutAudio[FrameIndex] += NoiseVolume * SynthBufferPtr[FrameIndex];
			}
		}
	}

	// Make sure we reach our target RPM exactly if we have no fade time
	if (CurrentRPMTime >= RPMFadeTime)
	{
		CurrentRPM = TargetRPM;
	}
}

int32 FMotoSynthEngine::OnGenerateAudio(float* OutAudio, int32 NumSamples)
{
	// Don't do anything if we're not enabled
	if (!IsMotoSynthEngineEnabled())
	{
		FMemory::Memzero(OutAudio, sizeof(float) * NumSamples);
		
		return NumSamples;
	}

	const int32 NumFrames = NumSamples / 2;

	// Generate granular audio w/ our mono buffer
	GrainEngineBuffer.Reset();
	GrainEngineBuffer.AddZeroed(NumFrames);

	GenerateGranularEngine(GrainEngineBuffer.GetData(), NumFrames);

	float* GrainEngineBufferPtr = GrainEngineBuffer.GetData();

	// Up-mix to dual-mono stereo
	for (int32 Frame = 0; Frame < NumFrames; ++Frame)
	{
		float GrainEngineSample = GrainEngineBufferPtr[Frame];
		for (int32 Channel = 0; Channel < 2; ++Channel)
		{
			OutAudio[Frame * 2 + Channel] = GrainEngineSample;
		}
	}

	if (bStereoWidenerEnabled)
	{
		// Feed through the stereo delay as "stereo widener"
		DelayStereo.ProcessAudio(OutAudio, NumSamples, OutAudio);
	}

	return NumSamples;
}

void FMotoSynthGrainRuntime::Init(const FGrainInitParams& InGrainInitParams)
{
	GrainEnvelope = InGrainInitParams.GrainEnvelope;
	GrainArrayView = InGrainInitParams.GrainView;
	NumBytesPerSample = InGrainInitParams.NumBytesPerSample;
	NumSamples = GrainArrayView.Num() / NumBytesPerSample;

	CurrentSampleIndex = 0.0f;
	FadeSamples = (float)InGrainInitParams.NumSamplesCrossfade;
	FadeOutStartIndex = (float)NumSamples - (float)FadeSamples;
	GrainPitchScale = 1.0f;
	EnginePitchScale = InGrainInitParams.EnginePitchScale;
	GrainRPMStart = InGrainInitParams.GrainStartRPM;
	GrainRPMDelta = InGrainInitParams.GrainEndRPM - InGrainInitParams.GrainStartRPM;
}

float FMotoSynthGrainRuntime::GenerateSample()
{
	if (CurrentSampleIndex >= NumSamples)
	{
		return 0.0f;
	}

	// compute the location of the grain playback in float-sample indices
	int32 PreviousSampleIndex = (int32)CurrentSampleIndex;
	int32 NextSampleIndex = PreviousSampleIndex + 1;

	if (NextSampleIndex < NumSamples)
	{
		float SampleValueInterpolated = 0.0f;
		if (NumBytesPerSample == 2)
		{
			int16* GrainDataPtr = (int16*)GrainArrayView.GetData();
			float PreviousSampleValue = (float)GrainDataPtr[PreviousSampleIndex] / TNumericLimits<int16>::Max();
			float NextSampleValue = (float)GrainDataPtr[NextSampleIndex] / TNumericLimits<int16>::Max();
			float SampleAlpha = CurrentSampleIndex - (float)PreviousSampleIndex;
			SampleValueInterpolated = FMath::Lerp(PreviousSampleValue, NextSampleValue, SampleAlpha);
		}
		else
		{
			// 8-bit data is polar 0.0 to 1.0, so scale to 0.0 to 2.0 and shift -1.0 to get back to bipolar (-1.0 to 1.0) float
			float PreviousSampleValue = (2.0f * (float)GrainArrayView[PreviousSampleIndex] / TNumericLimits<uint8>::Max()) - 1.0f;
			float NextSampleValue = (2.0f * (float)GrainArrayView[NextSampleIndex] / TNumericLimits<uint8>::Max()) - 1.0f;
			float SampleAlpha = CurrentSampleIndex - (float)PreviousSampleIndex;
			SampleValueInterpolated = FMath::Lerp(PreviousSampleValue, NextSampleValue, SampleAlpha);
		}

		// apply fade in or fade outs
		if (FadeSamples > 0)
		{
			if (CurrentSampleIndex < FadeSamples)
			{
				float FadeScale = (CurrentSampleIndex / FadeSamples);
				FadeScale = GrainEnvelope->GetValue(FadeScale * 0.5f);
				SampleValueInterpolated *= FadeScale;
			}
			else if (CurrentSampleIndex >= FadeOutStartIndex)
			{
				
				float FadeScale = FMath::Clamp(1.0f - ((CurrentSampleIndex - FadeOutStartIndex) / FadeSamples), 0.0f, 1.0f);
				FadeScale = GrainEnvelope->GetValue(0.5f * FadeScale);
				SampleValueInterpolated *= FadeScale;
			}		
		}

		// Update the pitch scale based on the progress through the grain and the starting and ending grain RPMs and the current runtime RPM
		float GrainFraction = CurrentSampleIndex / NumSamples;
		// Expected RPM given our playback progress, linearly interpolating the start and end RPMs
		float ExpectedRPM = GrainRPMStart + GrainFraction * GrainRPMDelta;
		GrainPitchScale = CurrentRuntimeRPM / ExpectedRPM;

		// Scale up the grain pitch scale according to the global engine pitch scale
		GrainPitchScale *= EnginePitchScale;

		CurrentSampleIndex += GrainPitchScale;
		return SampleValueInterpolated;
	}
	else
	{
		CurrentSampleIndex = (float)NumSamples + 1.0f;
	}

	return 0.0f;
}
bool FMotoSynthGrainRuntime::IsNearingEnd() const
{
	return (int32)CurrentSampleIndex >= (NumSamples - FadeSamples);
}

bool FMotoSynthGrainRuntime::IsDone() const
{
	return (int32)CurrentSampleIndex >= NumSamples;
}

void FMotoSynthGrainRuntime::SetRPM(int32 InRPM)
{
	CurrentRuntimeRPM = InRPM;
}
