// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotoSynthModule.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogMotoSynth);

IMPLEMENT_MODULE(FMotoSynthModule, MotoSynth)

void FMotoSynthModule::StartupModule()
{
	FModuleManager::Get().LoadModuleChecked(TEXT("SignalProcessing"));
}

void FMotoSynthModule::ShutdownModule()
{
}


