// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotoSynthPreset.h"
#include "AudioDevice.h"
#include "AudioMixerDevice.h"
#include "MotoSynthEngine.h"
#include "MotoSynthSourceAsset.h"
#include "MotoSynthModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotoSynthPreset)

void UMotoSynthPreset::BeginDestroy()
{
	Super::BeginDestroy();
}

void UMotoSynthPreset::PostLoad()
{
#if WITH_EDITORONLY_DATA
	if (Settings.SynthToneVolume_DEPRECATED > 0.0)
	{
		Settings.SynthToneVolumeRange = { Settings.SynthToneVolume_DEPRECATED, Settings.SynthToneVolume_DEPRECATED };
	}

	if (Settings.SynthToneFilterFrequency_DEPRECATED > 0.0f)
	{
		Settings.SynthToneFilterFrequencyRange = { Settings.SynthToneFilterFrequency_DEPRECATED, Settings.SynthToneFilterFrequency_DEPRECATED };
	}
#endif

	Super::PostLoad();
}

#if WITH_EDITOR
void UMotoSynthPreset::StartEnginePreview()
{
	// Set all the state of the previewer that needs setting
	EnginePreviewer->SetSettings(Settings);

	if (FRichCurve* RichRPMCurve = EnginePreviewRPMCurve.GetRichCurve())
	{
		EnginePreviewer->SetPreviewRPMCurve(*RichRPMCurve);
	}

	EnginePreviewer->StartPreviewing();
}

void UMotoSynthPreset::StopEnginePreview()
{
	EnginePreviewer->StopPreviewing();
}

void UMotoSynthPreset::DumpRuntimeMemoryUsage()
{
	float TotalMemMB = 0.0f;
	float AccelRuntime = 0.0f;
	float DecelRuntime = 0.0f;

	if (Settings.AccelerationSource)
	{
		AccelRuntime = Settings.AccelerationSource->GetRuntimeMemoryUsageMB();
	}

	if (Settings.DecelerationSource)
	{
		DecelRuntime = Settings.DecelerationSource->GetRuntimeMemoryUsageMB();
	}

	UE_LOGF(LogMotoSynth, Display, "Moto Synth Preset Memory: %ls, Accel: %.2f MB, Decel: %.2f MB, Total: %.2f MB", *GetName(), AccelRuntime, DecelRuntime, AccelRuntime + DecelRuntime);
}

void UMotoSynthPreset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName EnginePreviewRPMCurveFName = GET_MEMBER_NAME_CHECKED(UMotoSynthPreset, EnginePreviewRPMCurve);

	if (FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
	{
		const FName& Name = PropertyThatChanged->GetFName();
		if (Name == EnginePreviewRPMCurveFName)
		{
			if (FRichCurve* RichRPMCurve = EnginePreviewRPMCurve.GetRichCurve())
			{
				EnginePreviewer->SetPreviewRPMCurve(*RichRPMCurve);
			}
		}
		else
		{
			// Only set the settings on the engine previewer when we have both a acceleration and deceleration source
			if (Settings.AccelerationSource && Settings.DecelerationSource)
			{
				EnginePreviewer->SetSettings(Settings);
			}
			else
			{
				// Stop previewing if we've cleared out any sources (i.e. were previously previewing an accel/decel source, stop it now if one of them is null)
				EnginePreviewer->StopPreviewing();
			}
		}
	}
}
#endif

#if WITH_EDITOR
FMotoSynthEnginePreviewer::FMotoSynthEnginePreviewer()
{
	SynthEngine = TUniquePtr<FMotoSynthEngine>(new FMotoSynthEngine);
}

FMotoSynthEnginePreviewer::~FMotoSynthEnginePreviewer()
{
}

void FMotoSynthEnginePreviewer::SetSettings(const FMotoSynthRuntimeSettings& InSettings)
{
	FScopeLock Lock(&PreviewEngineCritSect);

	// Set the accel and decel data separately
	uint32 AccelDataID = INDEX_NONE;
	uint32 DecelDataID = INDEX_NONE;

	if (InSettings.AccelerationSource)
	{
		AccelDataID = InSettings.AccelerationSource->GetDataID();
	}

	if (InSettings.DecelerationSource)
	{
		DecelDataID = InSettings.DecelerationSource->GetDataID();
	}

	SynthEngine->SetSourceData(AccelDataID, DecelDataID);
	SynthEngine->GetRPMRange(RPMRange);

	Settings = InSettings;
	SynthEngine->SetSettings(InSettings);
}

void FMotoSynthEnginePreviewer::SetPreviewRPMCurve(const FRichCurve& InRPMCurve)
{
	FScopeLock Lock(&PreviewEngineCritSect);
	PreviewRPMCurve = InRPMCurve;
}

void FMotoSynthEnginePreviewer::StartPreviewing()
{
	bPreviewFinished = false;

	Reset();

	if (!bRegistered)
	{
		bEngineInitialized = false;
		if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
		{
			if (FAudioDevice* AudioDevice = AudioDeviceManager->GetMainAudioDeviceRaw())
			{
				bRegistered = true;
				AudioDevice->RegisterSubmixBufferListener(AsShared(), AudioDevice->GetMainSubmixObject());
			}
		}
	}
}

void FMotoSynthEnginePreviewer::StopPreviewing()
{
	bPreviewFinished = true;

	if (bRegistered)
	{
		if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
		{
			if (FAudioDevice* AudioDevice = AudioDeviceManager->GetMainAudioDeviceRaw())
			{
				bRegistered = false;
				bEngineInitialized = false;
				AudioDevice->UnregisterSubmixBufferListener(AsShared(), AudioDevice->GetMainSubmixObject());
			}
		}
	}
}

void FMotoSynthEnginePreviewer::OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 NumChannels, const int32 InSampleRate, double AudioClock)
{
	FScopeLock Lock(&PreviewEngineCritSect);

	if (bPreviewFinished)
	{
		return;
	}

	if (!bEngineInitialized)
	{
		bEngineInitialized = true;
		SynthEngine->Init(InSampleRate);
	}

	float RPMCurveTime = CurrentPreviewCurveTime;
	float MinTime = 0.0f;
	float MaxTime = 0.0f;
	PreviewRPMCurve.GetTimeRange(MinTime, MaxTime);

	// No way to preview RPMs if curve does not have a time range
	if (FMath::IsNearlyEqual(MinTime, MaxTime))
	{
		return;
	}

	// Update the clock of the previewer. Used to look up curves. 
	if (CurrentPreviewCurveStartTime == 0.0f || CurrentPreviewCurveTime >= MaxTime)
	{
		CurrentPreviewCurveStartTime = (float)AudioClock + MinTime;
		CurrentPreviewCurveTime = MinTime;
	}
	else
	{
		CurrentPreviewCurveTime = (float)AudioClock - CurrentPreviewCurveStartTime;
	}

	//UE_LOGF(LogTemp, Log, "RPM CURVE TIME: %.2f", CurrentPreviewCurveTime);

	// This should be a value between 0.0 and 1.0
	float CurrentRPMCurveValue = PreviewRPMCurve.Eval(CurrentPreviewCurveTime);

	// Normalize the value in the curve's range
	float ValueRangeMin;
	float ValueRangeMax;

	PreviewRPMCurve.GetValueRange(ValueRangeMin, ValueRangeMax);

	float FractionalValue = 0.0f;
	if (!FMath::IsNearlyEqual(ValueRangeMin, ValueRangeMax))
	{
		FractionalValue = (CurrentRPMCurveValue - ValueRangeMin) / (ValueRangeMax - ValueRangeMin);
	}

	float NextRPM = Audio::GetLogFrequencyClamped(FMath::Clamp(FractionalValue, 0.0f, 1.0f), { 0.0f, 1.0f }, RPMRange);
	SynthEngine->SetRPM(NextRPM);

	int32 NumFrames = NumSamples / NumChannels;

	// Generate the engine audio
	OutputBuffer.Reset();
	OutputBuffer.AddZeroed(NumFrames * 2);

	SynthEngine->GetNextBuffer(OutputBuffer.GetData(), NumFrames * 2, true);

	for (int32 FrameIndex = 0, SampleIndex = 0; FrameIndex < NumFrames; ++FrameIndex, SampleIndex += NumChannels)
	{
		for (int32 Channel = 0; Channel < 2; ++Channel)
		{
			AudioData[SampleIndex + Channel] += OutputBuffer[2 * FrameIndex + Channel];
		}
	}
}

const FString& FMotoSynthEnginePreviewer::GetListenerName() const
{
	static const FString ListenerName = TEXT("MotoSynthEnginePreviewer");
	return ListenerName;
}

void FMotoSynthEnginePreviewer::Reset()
{
	CurrentPreviewCurveStartTime = 0.0f;
	bEngineInitialized = false;
}
#endif // WITH_EDITOR
