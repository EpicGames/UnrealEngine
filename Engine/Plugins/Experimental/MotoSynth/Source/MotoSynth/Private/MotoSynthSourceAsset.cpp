// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotoSynthSourceAsset.h"

#include "MotoSynthModule.h"
#include "MotoSynthDataManager.h"
#include "Sound/SampleBufferIO.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "AudioDevice.h"
#include "DSP/DynamicsProcessor.h"
#include "DSP/FloatArrayMath.h"
#include "DSP/SampleRateConverter.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotoSynthSourceAsset)

///////////////////////////////////////////////////////////
// RPM estimation implementation
///////////////////////////////////////////////////////////
#if WITH_EDITOR
FRPMEstimationPreviewTone::FRPMEstimationPreviewTone()
{
}

FRPMEstimationPreviewTone::~FRPMEstimationPreviewTone()
{
	StopTestTone();
}

void FRPMEstimationPreviewTone::StartTestTone(float InVolume)
{
	VolumeScale = InVolume;
	if (!bRegistered)
	{
		if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
		{
			if (FAudioDevice* AudioDevice = AudioDeviceManager->GetMainAudioDeviceRaw())
			{
				bRegistered = true;
				AudioDevice->RegisterSubmixBufferListener(AsShared(), AudioDevice->GetMainSubmixObject());
			}
		}
	}
}

const FString& FRPMEstimationPreviewTone::GetListenerName() const
{
	static const FString ListenerName = TEXT("RPM Estimation Preview Tone Listener");
	return ListenerName;
}

void FRPMEstimationPreviewTone::StopTestTone()
{
	Reset();

	if (bRegistered)
	{
		if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
		{
			if (FAudioDevice* AudioDevice = AudioDeviceManager->GetMainAudioDeviceRaw())
			{
				bRegistered = false;
				AudioDevice->UnregisterSubmixBufferListener(AsShared(), AudioDevice->GetMainSubmixObject());
			}
		}
	}
}

void FRPMEstimationPreviewTone::Reset()
{
	FScopeLock Lock(&CallbackCritSect);
	AudioFileBuffer = TArrayView<const int16>();
	CurrentFrame = 0;
}

void FRPMEstimationPreviewTone::SetAudioFile(TArrayView<const int16>& InAudioFileBuffer, int32 InSampleRate)
{
	FScopeLock Lock(&CallbackCritSect);
	AudioFileBuffer = InAudioFileBuffer;

	SampleRate = InSampleRate;
	Osc.Init((float)InSampleRate);
	Osc.SetType(Audio::EOsc::Saw);
	Osc.SetGain(1.0f);
	Osc.Start();

	Filter.Init((float)InSampleRate, 1);
	Filter.SetFrequency(200.0f);
	Filter.Update();

	CurrentFrame = 0;
}

void FRPMEstimationPreviewTone::SetPitchCurve(FRichCurve& InRPMCurve)
{
	FScopeLock Lock(&CallbackCritSect);
	RPMCurve = InRPMCurve;
}

void FRPMEstimationPreviewTone::OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 NumChannels, const int32 InSampleRate, double AudioClock)
{
	FScopeLock Lock(&CallbackCritSect);

	if (CurrentFrame >= AudioFileBuffer.Num())
	{
		return;
	}

	int32 NumFrames = NumSamples / NumChannels;

	// Generate the test tone
	ScratchBuffer.Reset();
	ScratchBuffer.AddUninitialized(NumFrames);

	int32 ToneCurrentFrame = CurrentFrame;

	float* ScratchBufferPtr = ScratchBuffer.GetData();

	float LastFrequency = 0.0f;

	for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
	{
		float CurrentTime = (float)CurrentFrame / SampleRate;
		LastFrequency = RPMCurve.Eval(CurrentTime) / 60.0f;		

		Osc.SetFrequency(LastFrequency);
		Osc.Update();

		ScratchBufferPtr[FrameIndex] = VolumeScale * Osc.Generate();
	}

	Filter.SetFrequency(LastFrequency + 200.0f);
	Filter.Update();

	// Apply filter to test tone
	Filter.ProcessAudio(ScratchBuffer.GetData(), ScratchBuffer.Num(), ScratchBuffer.GetData());

	int32 SampleIndex = 0;
	for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
	{
		if (CurrentFrame >= AudioFileBuffer.Num())
		{
			return;
		}

		float SampleValue = (float)AudioFileBuffer[CurrentFrame++] / 32767.0f;
		float SampleOutput = SampleValue + ScratchBufferPtr[FrameIndex];

		// Left channel
		for (int32 Channel = 0; Channel < 2; ++Channel)
		{
			AudioData[SampleIndex + Channel] += SampleOutput;
		}

		SampleIndex += NumChannels;
	}
}
#endif

UMotoSynthSource::UMotoSynthSource()
#if WITH_EDITORONLY_DATA
	: MotoSynthSineToneTest(MakeShared<FRPMEstimationPreviewTone, ESPMode::ThreadSafe>())
#endif // WITH_EDITORONLY_DATA
{
}

UMotoSynthSource::~UMotoSynthSource()
{
}

void UMotoSynthSource::BeginDestroy()
{
	Super::BeginDestroy();

	// Unregister our data from the data manager
	if (SourceDataID != INDEX_NONE)
	{
		FMotoSynthSourceDataManager& MotoSynthDataManager = FMotoSynthSourceDataManager::Get();
		MotoSynthDataManager.UnRegisterData(SourceDataID);
		SourceDataID = INDEX_NONE;
	}

#if WITH_EDITORONLY_DATA
	MotoSynthSineToneTest->Reset();
	MotoSynthSineToneTest.Reset();
#endif // WITH_EDITORONLY_DATA
}

uint32 UMotoSynthSource::GetNextSourceID() const
{
	static uint32 SourceIDs = INDEX_NONE;
	return ++SourceIDs;
}

void UMotoSynthSource::RegisterSourceData()
{
	FMotoSynthSourceDataManager& MotoSynthDataManager = FMotoSynthSourceDataManager::Get();

	if (SourceDataID != INDEX_NONE)
	{
		MotoSynthDataManager.UnRegisterData(SourceDataID);
	}

	SourceDataID = GetNextSourceID();

	// We need a curve data
	FRichCurve* RichRPMCurve = RPMCurve.GetRichCurve();
	if (ensure(RichRPMCurve))
	{
		FName SourceName = GetFName();

		// In the editor, we don't want to modify the SourcDataPCM array
		// At runtime we want to move from it, which requires an unload and load to get back
#if WITH_EDITOR
		TArray<int16> DataCopy = SourceDataPCM;
		TArray<FGrainTableEntry> GrainTableCopy = GrainTable;
		MotoSynthDataManager.RegisterData(SourceDataID, SourceName, MoveTemp(DataCopy), SourceSampleRate, MoveTemp(GrainTableCopy), *RichRPMCurve, bConvertTo8Bit);
#else
		MotoSynthDataManager.RegisterData(SourceDataID, SourceName, MoveTemp(SourceDataPCM), SourceSampleRate, MoveTemp(GrainTable), *RichRPMCurve, bConvertTo8Bit);
#endif
	}
	else
	{
		UE_LOGF(LogMotoSynth, Error, "No RPM curve data loaded for moto synth source %ls", *GetName());
	}
}

void UMotoSynthSource::PostLoad()
{
	Super::PostLoad();

	if (GetOutermost()->HasAnyPackageFlags(PKG_ReloadingForCooker))
	{
		return;
	}

	if (SourceData_DEPRECATED.Num() > 0)
	{
		// Convert the previous source data to int16
		int32 NumFrames = SourceData_DEPRECATED.Num();
		SourceDataPCM.AddUninitialized(NumFrames);
		for (int32 Frame = 0; Frame < NumFrames; ++Frame)
		{
			SourceDataPCM[Frame] = (int16)(SourceData_DEPRECATED[Frame] * 32767.0f);	
		}
		SourceData_DEPRECATED.Reset();
	}

	RegisterSourceData();
}

#if WITH_EDITOR
void UMotoSynthSource::PlayToneMatch()
{
	MotoSynthSineToneTest->Reset();

	FRichCurve* RichRPMCurve = RPMCurve.GetRichCurve();
	if (RichRPMCurve)
	{
		MotoSynthSineToneTest->SetPitchCurve(*RichRPMCurve);
	}

	TArrayView<const int16> Audiobuffer = MakeArrayView(SourceDataPCM.GetData(), SourceDataPCM.Num());
	MotoSynthSineToneTest->SetAudioFile(Audiobuffer, SourceSampleRate);

	MotoSynthSineToneTest->StartTestTone(RPMSynthVolume);
}

void UMotoSynthSource::StopToneMatch()
{
	MotoSynthSineToneTest->StopTestTone();
}

void UMotoSynthSource::UpdateSourceData()
{
	if (!SoundWaveSource)
	{
		return;
	}

	TArray<uint8> ImportedSoundWaveData;
	uint32 ImportedSampleRate;
	uint16 ImportedChannelCount;
	SoundWaveSource->GetImportedSoundWaveData(ImportedSoundWaveData, ImportedSampleRate, ImportedChannelCount);

	SourceSampleRate = ImportedSampleRate;

	const int32 NumFrames = (ImportedSoundWaveData.Num() / sizeof(int16)) / ImportedChannelCount;

	SourceDataPCM.Reset();

	int16* ImportedDataPtr = (int16*)ImportedSoundWaveData.GetData();

	if (ImportedChannelCount == 1)
	{
		SourceDataPCM.Append(ImportedDataPtr, NumFrames);
	}
	else
	{
		SourceDataPCM.AddUninitialized(NumFrames);
		int16* RawSourceDataPtr = SourceDataPCM.GetData();

		// only use the left-channel
		for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			const int32 SampleIndex = FrameIndex * ImportedChannelCount;
			RawSourceDataPtr[FrameIndex] = ImportedDataPtr[SampleIndex];
		}
	}

	// Perform downsample if there is a downsample factor specified
	if (DownSampleFactor < 1.0f)
	{
		Audio::ISampleRateConverter* SRC = Audio::ISampleRateConverter::CreateSampleRateConverter();
		SRC->Init(1.0f / DownSampleFactor, 1);

		TArray<float> DownsampledImportedData;
		SRC->ProcessFullbuffer(SourceDataPCM.GetData(), SourceDataPCM.Num(), DownsampledImportedData);

		SourceDataPCM.Reset();
		for (int32 FrameIndex = 0; FrameIndex < DownsampledImportedData.Num(); ++FrameIndex)
		{
			int16 SampleValue = TNumericLimits<int16>::Max() * DownsampledImportedData[FrameIndex];
			SourceDataPCM.Add(SampleValue);
		}

		SourceSampleRate *= DownSampleFactor;
		delete SRC;
	}
}

void UMotoSynthSource::FilterSourceDataForAnalysis()
{
	Audio::FAlignedFloatBuffer ScratchBuffer;
	ScratchBuffer.AddUninitialized(SourceDataPCM.Num());

	float* ScratchDataBufferPtr = ScratchBuffer.GetData();

	// Convert the source data to floats
	Audio::ArrayPcm16ToFloat(MakeArrayView(SourceDataPCM), MakeArrayView(ScratchDataBufferPtr, SourceDataPCM.Num()));

	// Filter the audio source and write the output to the analysis buffer. Do not modify the source audio data.
	if (bEnableFilteringForAnalysis)
	{
		Audio::FBiquadFilter HPF;
		HPF.Init(SourceSampleRate, 1, Audio::EBiquadFilter::Highpass, HighPassFilterFrequency);
		HPF.ProcessAudio(ScratchDataBufferPtr, ScratchBuffer.Num(), ScratchDataBufferPtr);

		AnalysisBuffer = ScratchBuffer;

		Audio::FBiquadFilter LPF;
		LPF.Init(SourceSampleRate, 1, Audio::EBiquadFilter::Lowpass, LowPassFilterFrequency);
		LPF.ProcessAudio(ScratchDataBufferPtr, AnalysisBuffer.Num(), ScratchDataBufferPtr);
	}

	// Move from the scratch buffer
	AnalysisBuffer = MoveTemp(ScratchBuffer);
}

void UMotoSynthSource::DynamicsProcessForAnalysis()
{
	// Feed the filtered audio file through a dynamics processor to get a common audio amplitude profile
	if (bEnableDynamicsProcessorForAnalysis)
	{
		Audio::FDynamicsProcessor DynamicsProcessor;
		DynamicsProcessor.Init(SourceSampleRate, 1);

		DynamicsProcessor.SetLookaheadMsec(DynamicsProcessorLookahead);
		DynamicsProcessor.SetInputGain(DynamicsProcessorInputGainDb);
		DynamicsProcessor.SetRatio(DynamicsProcessorRatio);
		DynamicsProcessor.SetKneeBandwidth(DynamicsKneeBandwidth);
		DynamicsProcessor.SetThreshold(DynamicsProcessorThreshold);
		DynamicsProcessor.SetAttackTime(DynamicsProcessorAttackTimeMsec);
		DynamicsProcessor.SetReleaseTime(DynamicsProcessorReleaseTimeMsec);

		Audio::FAlignedFloatBuffer DynamicsProcessorScratchBuffer;
		DynamicsProcessorScratchBuffer.AddUninitialized(AnalysisBuffer.Num());

		DynamicsProcessor.ProcessAudio(AnalysisBuffer.GetData(), AnalysisBuffer.Num(), DynamicsProcessorScratchBuffer.GetData());

		AnalysisBuffer = MoveTemp(DynamicsProcessorScratchBuffer);
	}
}

void UMotoSynthSource::NormalizeForAnalysis()
{
	if (bEnableNormalizationForAnalysis)
	{
		// Find max abs sample
		float MaxSample = 0.0f;
		for (float& Sample : AnalysisBuffer)
		{
			MaxSample = FMath::Max(MaxSample, FMath::Abs(Sample));
		}

		// If we found one, which we should, use to normalize the signal
		if (MaxSample > 0.0f)
		{
			Audio::ArrayMultiplyByConstantInPlace(AnalysisBuffer, 1.0f / MaxSample);
		}
	}
}

void UMotoSynthSource::BuildGrainTableByFFT()
{
	// TODO
}

static void GetBufferViewFromAnalysisBuffer(const Audio::FAlignedFloatBuffer& InAnalysisBuffer, int32 StartingBufferIndex, int32 BufferSize, TArrayView<const float>& OutBufferView)
{
	if (BufferSize > 0)
	{
		BufferSize = FMath::Min(BufferSize, InAnalysisBuffer.Num() - StartingBufferIndex);
		OutBufferView = MakeArrayView(&InAnalysisBuffer[StartingBufferIndex], BufferSize);
	}
}

static float ComputeCrossCorrelation(const TArrayView<const float>& InBufferA, const TArrayView<const float>& InBufferB)
{
	float SumA = 0.0f;
	float SumB = 0.0f;
	float SumAB = 0.0f;
	float SquareSumA = 0.0f;
	float SquareSumB = 0.0f;
	int32 Num = InBufferA.Num();

	for (int32 Index = 0; Index < Num; ++Index)
	{
		// scale the InBufferB to match InBufferA via linear sample rate conversion
		float FractionalIndex = ((float)Index / Num) * InBufferB.Num();
		int32 BIndexPrev = (int32)FractionalIndex;
		int32 BIndexNext = FMath::Min(BIndexPrev + 1, InBufferB.Num() - 1);

		float BufferBSample = FMath::Lerp(InBufferB[BIndexPrev], InBufferB[BIndexNext], FractionalIndex - (float)BIndexPrev);

		SumA += InBufferA[Index];
		SumB += BufferBSample;
		SumAB += InBufferA[Index] * BufferBSample;

		SquareSumA += (InBufferA[Index] * InBufferA[Index]);
		SquareSumB += (BufferBSample * BufferBSample);
	}

	float CorrelationNumerator = (Num * SumAB - SumA * SumB);
	float CorrelationDenomenator = FMath::Sqrt((Num * SquareSumA - SumA * SumA) * (Num * SquareSumB - SumB * SumB));
	check(CorrelationDenomenator > 0.0f);
	float Correlation = CorrelationNumerator / CorrelationDenomenator;
	return Correlation;
}

void UMotoSynthSource::BuildGrainTableByRPMEstimation()
{
	// Reset the graintable in case this is a re-generate call
	GrainTable.Reset();

	if (!AnalysisBuffer.Num())
	{
		UE_LOGF(LogMotoSynth, Error, "No analysis buffer to build grain table from.");
		return;
	}

	if (SourceSampleRate <= 0)
	{
		UE_LOGF(LogMotoSynth, Error, "Unable to build grain table for moto synth soruce, source sample rate is invalid (%d)", SourceSampleRate);
		return;
	}

	float DeltaTime = 1.0f / SourceSampleRate;
	float CurrentPhase = 1.0f;
	int32 CurrentSampleIndex = RPMCycleCalibrationSample;
		
	// Cycle back to the beginning of the file with this sample as the calibration sample
	while (CurrentSampleIndex >= 1)
	{
		CurrentPhase = PI;
		while (CurrentPhase >= 0.0f && CurrentSampleIndex >= 0)
		{
			float w0 = GetCurrentRPMForSampleIndex(CurrentSampleIndex) / 60.0f;
			float w1 = GetCurrentRPMForSampleIndex(CurrentSampleIndex--) / 60.0f;
			float Alpha = (w1 - w0) / DeltaTime;
			float DeltaPhase = w0 * DeltaTime + 0.5f * Alpha * DeltaTime * DeltaTime;
			CurrentPhase -= DeltaPhase;
		}
	}

	// Now, where the 'current phase' is, we can read forward from current sample index.
	float* AnalysisBufferPtr = AnalysisBuffer.GetData();
	while (CurrentSampleIndex < AnalysisBuffer.Num() - 1)
	{
		// We read through samples accumulating phase until the phase is greater than 1.0
		// That will indicate the need make a grain entry
		while (CurrentPhase < PI && CurrentSampleIndex < AnalysisBuffer.Num() - 1)
		{
			float w0 = GetCurrentRPMForSampleIndex(CurrentSampleIndex) / 60.0f;
			float w1 = GetCurrentRPMForSampleIndex(CurrentSampleIndex++) / 60.0f;
			float Alpha = (w1 - w0) / DeltaTime;
			float DeltaPhase = w0 * DeltaTime + 0.5f * Alpha * DeltaTime * DeltaTime;
			CurrentPhase += DeltaPhase;
		}

		CurrentPhase = 0.0f;

		// Immediately make a grain table entry for the beginning of the asset
		FGrainTableEntry NewGrain;
		NewGrain.RPM = GetCurrentRPMForSampleIndex(CurrentSampleIndex);;
		NewGrain.AnalysisSampleIndex = CurrentSampleIndex;
		NewGrain.SampleIndex = FMath::Max(NewGrain.AnalysisSampleIndex - SampleShiftOffset, 0);
		GrainTable.Add(NewGrain);
	}

	UE_LOGF(LogMotoSynth, Log, "Grain Table Built Using RPM Estimation: %d Grains", GrainTable.Num());
}

void UMotoSynthSource::WriteDebugDataToWaveFiles()
{
	if (bWriteAnalysisInputToFile && !AnalysisInputFilePath.IsEmpty())
	{
		WriteAnalysisBufferToWaveFile();
		WriteGrainTableDataToWaveFile();
	}
}


void UMotoSynthSource::PerformGrainTableAnalysis()
{
	UpdateSourceData();
	FilterSourceDataForAnalysis();
	DynamicsProcessForAnalysis();
	NormalizeForAnalysis();
	BuildGrainTableByRPMEstimation();
	WriteDebugDataToWaveFiles();

	RegisterSourceData();

	AnalysisBuffer.Reset();
}

void UMotoSynthSource::WriteAnalysisBufferToWaveFile()
{
	if (AnalysisBuffer.Num() > 0)
	{
		// Write out the analysis buffer
		Audio::FSoundWavePCMWriter Writer;

		TArray<int16> AnalysisBufferInt16;
		AnalysisBufferInt16.AddUninitialized(AnalysisBuffer.Num());

		Audio::ArrayFloatToPcm16(MakeArrayView(AnalysisBuffer), MakeArrayView(AnalysisBufferInt16));

		Audio::TSampleBuffer<> BufferToWrite(AnalysisBufferInt16.GetData(), AnalysisBufferInt16.Num(), 1, SourceSampleRate);

		FString FileName = FString::Printf(TEXT("%s_Analysis"), *GetName());
		Writer.BeginWriteToWavFile(BufferToWrite, FileName, AnalysisInputFilePath);
	}
}

void UMotoSynthSource::WriteGrainTableDataToWaveFile()
{
	if (GrainTable.Num() > 0)
	{
		int32 BufferSize = SourceDataPCM.Num();
		check(BufferSize == AnalysisBuffer.Num());

		TArray<int16> AnalysisGrainTableBufferInt16;
		AnalysisGrainTableBufferInt16.AddZeroed(BufferSize);

		TArray<int16> SourceGrainTableBufferInt16;
		SourceGrainTableBufferInt16.AddZeroed(BufferSize);

		// Write out the grain table for the analysis file
		for (int32 GrainIndex = 0; GrainIndex < GrainTable.Num(); ++GrainIndex)
		{
			FGrainTableEntry& Entry = GrainTable[GrainIndex];

			int32 AnalysisGrainSampleDuration = 0;
			int32 SoruceGrainSampleDuration = 0;

			// Write out the audio storeed in the grain table
			if (GrainIndex == GrainTable.Num() - 1)
			{
				AnalysisGrainSampleDuration = AnalysisBuffer.Num() - Entry.AnalysisSampleIndex;
				SoruceGrainSampleDuration = SourceDataPCM.Num() - Entry.SampleIndex;
			}
			else
			{
				AnalysisGrainSampleDuration = GrainTable[GrainIndex + 1].AnalysisSampleIndex - Entry.AnalysisSampleIndex;
				SoruceGrainSampleDuration = GrainTable[GrainIndex + 1].SampleIndex - Entry.SampleIndex;
			}

			for (int32 AudioDataIndex = 0; AudioDataIndex < AnalysisGrainSampleDuration; ++AudioDataIndex)
			{
				int32 BufferIndex = Entry.AnalysisSampleIndex + AudioDataIndex;
				if (BufferIndex < AnalysisBuffer.Num())
				{
					float SampleDataData = 1.0f;

					if (AudioDataIndex != 0)
					{
						SampleDataData = AnalysisBuffer[BufferIndex];
					}

					AnalysisGrainTableBufferInt16[Entry.AnalysisSampleIndex + AudioDataIndex] = SampleDataData * 32767.0f;
				}
			}

			for (int32 AudioDataIndex = 0; AudioDataIndex < SoruceGrainSampleDuration; ++AudioDataIndex)
			{
				int32 BufferIndex = Entry.SampleIndex + AudioDataIndex;
				if (BufferIndex < SourceDataPCM.Num())
				{
					float SampleDataData = 1.0f;

					if (AudioDataIndex != 0)
					{
						SampleDataData = (float)SourceDataPCM[BufferIndex] / 32767.0f;
					}

					SourceGrainTableBufferInt16[Entry.SampleIndex + AudioDataIndex] = SampleDataData * 32767.0f;
				}
			}
		}

		{
			Audio::FSoundWavePCMWriter Writer;
			Audio::TSampleBuffer<> BufferToWrite(AnalysisGrainTableBufferInt16.GetData(), AnalysisGrainTableBufferInt16.Num(), 1, SourceSampleRate);
			FString FileName = FString::Printf(TEXT("%s_AnalysisGrains"), *GetName());
			Writer.BeginWriteToWavFile(BufferToWrite, FileName, AnalysisInputFilePath);
		}

		{
			Audio::FSoundWavePCMWriter Writer;
			Audio::TSampleBuffer<> BufferToWrite(SourceGrainTableBufferInt16.GetData(), SourceGrainTableBufferInt16.Num(), 1, SourceSampleRate);
			FString FileName = FString::Printf(TEXT("%s_SourceGrains"), *GetName());
			Writer.BeginWriteToWavFile(BufferToWrite, FileName, AnalysisInputFilePath);
		}
	}
}

float UMotoSynthSource::GetCurrentRPMForSampleIndex(int32 CurrentSampleIndex)
{
	float CurrentTimeSec = (float)CurrentSampleIndex / (float)SourceSampleRate;
	FRichCurve* RichRPMCurve = RPMCurve.GetRichCurve();
	float CurveValue = RichRPMCurve->Eval(CurrentTimeSec);
	return CurveValue;
}

void UMotoSynthSource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		bool bUpdated = false;
		if (FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
		{
			const FName& Name = PropertyThatChanged->GetFName();
			static FName DownSampleFactorName = GET_MEMBER_NAME_CHECKED(UMotoSynthSource, DownSampleFactor);
			if (Name == DownSampleFactorName)
			{
				PerformGrainTableAnalysis();
				bUpdated = true;
			}
		}

		if (!bUpdated)
		{
			RegisterSourceData();
		}
	}
}
#endif // #if WITH_EDITOR


float UMotoSynthSource::GetRuntimeMemoryUsageMB() const
{
	if (SourceDataID != INDEX_NONE)
	{
		FMotoSynthSourceDataManager& MotoSynthDataManager = FMotoSynthSourceDataManager::Get();
		MotoSynthDataPtr MotoSynthData = MotoSynthDataManager.GetMotoSynthData(SourceDataID);
		
		int32 TotalMemBytes = MotoSynthData->GrainTable.Num() * sizeof(FGrainTableEntry);
		if (MotoSynthData->AudioSourceBitCrushed.Num() > 0)
		{
			TotalMemBytes += MotoSynthData->AudioSourceBitCrushed.Num();
		}
		else
		{
			TotalMemBytes += MotoSynthData->AudioSource.Num() * sizeof(int16);
		}

		return (float)TotalMemBytes / (1024.0f * 1024.0f);
	}
	// SourceData isn't loaded so no memory usage
	return 0.0f;
}

