// Copyright Epic Games, Inc. All Rights Reserved.


#include "SynthComponents/SynthComponentMoto.h"
#include "MotoSynthSourceAsset.h"
#include "MotoSynthEngine.h"
#include "MotoSynthModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SynthComponentMoto)


USynthComponentMoto::USynthComponentMoto(const FObjectInitializer& ObjInitializer)
	: Super(ObjInitializer)
{
	// Moto synth upmixes mono to stereo
	NumChannels = 2;
}

USynthComponentMoto::~USynthComponentMoto()
{
}

bool USynthComponentMoto::IsEnabled() const
{
	return FMotoSynthEngine::IsMotoSynthEngineEnabled();
}

void USynthComponentMoto::SetRPM(float InRPM, float InTimeSec)
{
	if (FMotoSynthEngine::IsMotoSynthEngineEnabled())
	{
		if (InRPM > KINDA_SMALL_NUMBER && !FMath::IsNaN(InRPM))
		{
			RPM = InRPM;
			if (MotoSynthEngine.IsValid())
			{
				if (FMotoSynthEngine* MS = static_cast<FMotoSynthEngine*>(MotoSynthEngine.Get()))
				{
					float NewRPM = FMath::Clamp(InRPM, RPMRange.X, RPMRange.Y);
					MS->SetRPM(NewRPM, InTimeSec);
				}
			}
		}
		else
		{
			UE_LOGF(LogMotoSynth, Verbose, "Moto synth SetRPM was given invalid RPM value: %f.", InRPM);
		}
	}
}

void USynthComponentMoto::SetSettings(const FMotoSynthRuntimeSettings& InSettings)
{
	FScopeLock ScopeLock(&SettingsCriticalSection);

	bSettingsOverridden = true;
	OverrideSettings = InSettings;

	if (FMotoSynthEngine* MS = static_cast<FMotoSynthEngine*>(MotoSynthEngine.Get()))
	{
		uint32 AccelDataID = OverrideSettings.AccelerationSource->GetDataID();
		uint32 DecelDataID = OverrideSettings.DecelerationSource->GetDataID();

		MS->SetSourceData(AccelDataID, DecelDataID);
		MS->SetSettings(OverrideSettings);
	}
}

void USynthComponentMoto::GetRPMRange(float& OutMinRPM, float& OutMaxRPM)
{
	if (FMath::IsNearlyZero(RPMRange.X) && FMath::IsNearlyZero(RPMRange.Y))
	{
		FScopeLock ScopeLock(&SettingsCriticalSection);

		FMotoSynthRuntimeSettings* Settings = GetSettingsToUse();
		if (Settings)
		{
			FRichCurve* AccelRichCurve = Settings->AccelerationSource->RPMCurve.GetRichCurve();
			FRichCurve* DecelRichCurve = Settings->DecelerationSource->RPMCurve.GetRichCurve();
			if (AccelRichCurve && DecelRichCurve)
			{
				FVector2f AccelRPMRange;
				AccelRichCurve->GetValueRange(AccelRPMRange.X, AccelRPMRange.Y);

				FVector2f DecelRPMRange;
				DecelRichCurve->GetValueRange(DecelRPMRange.X, DecelRPMRange.Y);

				RPMRange = { FMath::Max(AccelRPMRange.X, DecelRPMRange.X), FMath::Min(AccelRPMRange.Y, DecelRPMRange.Y) };
			}
		}	
	}

	OutMinRPM = RPMRange.X;
	OutMaxRPM = RPMRange.Y;

	if (FMath::IsNearlyEqual(OutMinRPM, OutMaxRPM))
	{
		UE_LOGF(LogMotoSynth, Verbose, "Moto synth min and max RPMs are nearly identical. Min RPM: %f, Max RPM: %f", OutMinRPM, OutMaxRPM);
		OutMaxRPM = OutMinRPM + 1.0f;
	}
}

FMotoSynthRuntimeSettings* USynthComponentMoto::GetSettingsToUse()
{
	FMotoSynthRuntimeSettings* Settings = nullptr;
	if (!bSettingsOverridden && MotoSynthPreset && MotoSynthPreset->Settings.AccelerationSource && MotoSynthPreset->Settings.DecelerationSource)
	{
		Settings = &MotoSynthPreset->Settings;
	}
	else if (bSettingsOverridden && OverrideSettings.AccelerationSource && OverrideSettings.DecelerationSource)
	{
		Settings = &OverrideSettings;
	}
	else
	{
		UE_LOGF(LogMotoSynth, Verbose, "Invalid moto synth preset or missing acceleration or deceleraton source.");
	}

	return Settings;
}

ISoundGeneratorPtr USynthComponentMoto::CreateSoundGenerator(const FSoundGeneratorInitParams& InParams)
{
	if (!FMotoSynthEngine::IsMotoSynthEngineEnabled())
	{
		UE_LOGF(LogMotoSynth, Verbose, "Moto synth has been disabled by cvar.");
		return ISoundGeneratorPtr(new FSoundGeneratorNull());
	}
 
	FScopeLock ScopeLock(&SettingsCriticalSection);

	FMotoSynthRuntimeSettings* Settings = GetSettingsToUse();
	if (Settings)
	{
 		MotoSynthEngine = ISoundGeneratorPtr(new FMotoSynthEngine());
 
 		if (FMotoSynthEngine* MS = static_cast<FMotoSynthEngine*>(MotoSynthEngine.Get()))
 		{
 			MS->Init(InParams.SampleRate);
 
			uint32 AccelDataID = Settings->AccelerationSource->GetDataID();
			uint32 DecelDataID = Settings->DecelerationSource->GetDataID();
 
 			MS->SetSourceData(AccelDataID, DecelDataID);
			MS->SetSettings(*Settings);
 		}
 
 		return MotoSynthEngine;
 	}
 	else
 	{
 		UE_LOGF(LogMotoSynth, Verbose, "Can't play moto synth without a preset UMotoSynthPreset object and both acceleration source and deceleration source set.");
 		return ISoundGeneratorPtr(new FSoundGeneratorNull());
 	}
}
