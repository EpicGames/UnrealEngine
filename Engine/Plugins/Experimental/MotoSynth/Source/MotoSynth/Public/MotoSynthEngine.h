// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundGenerator.h"
#include "DSP/DelayStereo.h"
#include "DSP/Granulator.h"
#include "MotoSynthDataManager.h"

#define UE_API MOTOSYNTH_API

struct FMotoSynthRuntimeSettings;

class UMotoSynthSource;
class FMotoSynthEnginePreviewer;

class FMotoSynthAssetManager
{
public:

	// Retrieves the moto synth asset manager
	static FMotoSynthAssetManager& Get();

	FMotoSynthAssetManager();
	~FMotoSynthAssetManager();
};

struct FGrainInitParams
{
	const Audio::FGrainEnvelope* GrainEnvelope = nullptr;
	TArrayView<const uint8> GrainView;
	uint8 NumBytesPerSample = 0;
	int32 NumSamplesCrossfade = 0;
	float GrainStartRPM = 0.0f;
	float GrainEndRPM = 0.0f;
	float StartingRPM = 0.0f;
	float EnginePitchScale = 0.0f;
};

class FMotoSynthGrainRuntime
{
public:
	void Init(const FGrainInitParams& InInitParams);

	// Generates a sample from the grain. Returns true if fading out
	float GenerateSample();

	// Returns true if the grain is nearing its end (and a new grain needs to start fading in
	bool IsNearingEnd() const;

	// If the grain is done
	bool IsDone() const;

	// Updates the grain's RPM
	void SetRPM(int32 InRPM);

private:
	const Audio::FGrainEnvelope* GrainEnvelope = nullptr;
	TArrayView<const uint8> GrainArrayView;
	float CurrentSampleIndex = 0.0f;
	float FadeSamples = 0;
	float FadeOutStartIndex = 0.0f;
	float GrainPitchScale = 1.0f;
	float EnginePitchScale = 1.0f;
	float CurrentRuntimeRPM = 0.0f;
	float GrainRPMStart = 0.0f;
	float GrainRPMDelta = 0.0f;
	float StartingRPM = 0.0f;
	uint8 NumBytesPerSample = 2;
	int32 NumSamples = 0;
};

class IMotoSynthEngine
{
public:
	virtual void SetSettings(const FMotoSynthRuntimeSettings& InSettings) = 0;
};

// Class for granulating an engine
class FMotoSynthEngine : public ISoundGenerator, 
									   public IMotoSynthEngine
{
public:
	UE_API FMotoSynthEngine();
	UE_API ~FMotoSynthEngine();

	// Queries if the engine is enabled at all. Checks a cvar.
	static UE_API bool IsMotoSynthEngineEnabled();

	//~ Begin IMotoSynthEngine
	UE_API void SetSettings(const FMotoSynthRuntimeSettings& InSettings) final;
	//~ End IMotoSynthEngine

	//~ Begin FSoundGenerator 
	virtual int32 GetDesiredNumSamplesToRenderPerCallback() const { return 256; }
	UE_API virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) override;
	//~ End FSoundGenerator

	UE_API void Init(int32 InSampleRate);
	UE_API void Reset();

	// Sets all the source data for the moto synth
	UE_API void SetSourceData(uint32 AccelerationSourceID, uint32 DecelerationSourceID);

	// Returns the min and max RPM range, taking into account the acceleration and deceleration data.
	UE_API void GetRPMRange(FVector2D& OutRPMRange);

	// Sets the RPM directly. Used if the engine is in ManualRPM mode. Will be ignored if we're in simulation mode.
	UE_API void SetRPM(float InRPM, float InTimeSec = 0.0f);

	// Sets a pitch scale on the moto synth to scale up or down the pitch of the output
	UE_API void SetPitchScale(float InPitchScale);

private:
	UE_API void GenerateGranularEngine(float* OutAudio, int32 NumSamples);

	UE_API bool NeedsSpawnGrain();
	UE_API void SpawnGrain(int32& StartingIndex, const MotoSynthDataPtr& SynthData);

	float RendererSampleRate = 0.0f;

	// The current RPM state
	float CurrentRPM = 0.0f;
	float PreviousRPM = 0.0f;
	float CurrentRPMSlope = 0.0f;
	float PreviousRPMSlope = 0.0f;
	float TargetRPM = 0.0f;
	float StartingRPM = 0.0f;
	float RPMFadeTime = 0.0f;
	float CurrentRPMTime = 0.0f;
	float PitchScale = 1.0f;

	int32 CurrentAccelerationSourceDataIndex = 0;
	int32 CurrentDecelerationSourceDataIndex = 0;

	// The source data
	MotoSynthDataPtr AccelerationSourceData;
	MotoSynthDataPtr DecelerationSourceData;

	FVector2f RPMRange;
	FVector2f RPMRange_RendererCallback;

	// Number of samples to use to do a grain crossfade. Smooths out discontinuities on grain boundaries.
	int32 GrainCrossfadeSamples = 10;
	int32 NumGrainTableEntriesPerGrain = 3;
	int32 GrainTableRandomOffsetForConstantRPMs = 20;
	int32 GrainCrossfadeSamplesForConstantRPMs = 40;

	// The grain pool for runtime generation of audio
	TArray<FMotoSynthGrainRuntime> GrainPool;

	// The grain state management arrays
	TArray<int32> ActiveGrains; // Grains actively generating and not fading out
	TArray<int32> FreeGrains; // Grain indicies which are free to be used. max size should be equal to grain pool size.

	TArray<float> SynthBuffer;
	FVector2f SynthFilterFreqRange = { 100.0f, 5000.0f };
	Audio::FLadderFilter SynthFilter;
	Audio::FOsc SynthOsc;
	Audio::FADEnvelope SynthEnv;
	Audio::AlignedFloatBuffer SynthEnvBuffer;

	Audio::FBiquadFilter NoiseFilter;
	Audio::FWhiteNoise NoiseGen;
	Audio::FADEnvelope NoiseEnv;
	Audio::AlignedFloatBuffer NoiseEnvBuffer;

	int32 SynthPitchUpdateSampleIndex = 0;
	int32 SynthPitchUpdateDeltaSamples = 1023;

	// Stereo delay to "widen" the moto synth output
	Audio::FDelayStereo DelayStereo;

	// Grain Envelope buffer
	Audio::FGrainEnvelope GrainEnvelope;

	// Mono scratch buffer for engine generation
	TArray<float> GrainEngineBuffer;

	FVector2f SynthToneVolumeRange = { 0.0f, 0.0f };
	FVector2f SynthToneFilterFrequencyRange = { 500.0f, 500.0f };
	FVector2f SynthToneAttackTimeMsecRange = { 10.0f, 10.0f };
	FVector2f SynthToneDecayTimeMsecRange = { 100.0f, 100.0f };
	FVector2f SynthToneAttackCurveRange = { 1.0f, 1.0f };
	FVector2f SynthToneDecayCurveRange = { 1.0f, 1.0f };

	FVector2f NoiseVolumeRange = { 0.0f, 0.0f };
	FVector2f NoiseLPFRange = { 0.0f, 0.0f };
	FVector2f NoiseAttackTimeMsecRange = { 10.0f, 10.0f };
	FVector2f NoiseAttackCurveRange = { 1.0f, 1.0f };
	FVector2f NoiseDecayTimeMsecRange = { 10.0f, 10.0f };
	FVector2f NoiseDecayCurveRange = { 1.0f, 1.0f };

	int32 SynthOctaveShift = 0;
	float GranularEngineVolume = 1.0f;
	float TargetGranularEngineVolume = 1.0f;

	bool bWasAccelerating = false;
	bool bSynthToneEnabled = false;
	bool bSynthToneEnvelopeEnabled = false;
	bool bNoiseEnabled = false;
	bool bNoiseEnvelopeEnabled = false;
	bool bGranularEngineEnabled = true;
	bool bStereoWidenerEnabled = true;
	bool bRPMWasSet = false;
	bool bUpdateRPMCalculations = false;
};

#undef UE_API
