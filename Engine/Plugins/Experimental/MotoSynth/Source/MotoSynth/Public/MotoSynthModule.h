// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMotoSynth, Log, All);

class FMotoSynthModule : public IModuleInterface
{
public:

	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
