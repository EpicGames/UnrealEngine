// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/CurveFloat.h"
#include "ISubmixBufferListener.h"
#include "MotoSynthPreset.generated.h"

#define UE_API MOTOSYNTH_API

class UMotoSynthPreset;
struct FPropertyChangedEvent;
class UMotoSynthSource;
class FMotoSynthEngine;

USTRUCT(BlueprintType)
struct FMotoSynthRuntimeSettings
{
	GENERATED_USTRUCT_BODY()

	// If the synth tone is enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth")
	bool bSynthToneEnabled = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float SynthToneVolume_DEPRECATED = 0.0f;

	UPROPERTY()
	float SynthToneFilterFrequency_DEPRECATED = 0.0f;
#endif

	// The volume of the synth tone	between min and max RPM
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (EditCondition = "bSynthToneEnabled"))
	FVector2D SynthToneVolumeRange = { 0.0f, 0.0f };

	// The filter frequency of the synth tone between min/max RPM
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (EditCondition = "bSynthToneEnabled"))
	FVector2D SynthToneFilterFrequencyRange = { 500.0f, 500.0f };

	// Enable the synth tone envelope generator
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (EditCondition = "bSynthToneEnabled"))
	bool bSynthToneEnvelopeEnabled = true;

	// Synth tone envelope attack time in milliseconds between min/max RPM
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (EditCondition = "bSynthToneEnabled"))
	FVector2D SynthToneAttackTimeMsecRange = { 10.0f, 10.0f };

	// Synth tone envelope attack curve.  1.0 = linear growth, < 1.0 logorithmic growth, > 1.0 exponential growth.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (EditCondition = "bSynthToneEnabled"))
	FVector2D SynthToneAttackCurveRange = { 1.0f, 1.0f };

	// Synth tone decay time in milliseconds
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (EditCondition = "bSynthToneEnabled"))
	FVector2D SynthToneDecayTimeMsecRange = { 100.0f, 100.0f };

	// Synth tone envelope decay curve.  1.0 = linear decay, < 1.0 exponential decay, > 1.0 logarithmic decay.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (EditCondition = "bSynthToneEnabled"))
	FVector2D SynthToneDecayCurveRange = { 1.0f, 1.0f };

	// Octave shift of the synth
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (ClampMin = "-3", ClampMax = "3", UIMin = "-3", UIMax = "3", EditCondition = "bSynthToneEnabled"))
	int32 SynthOctaveShift = 0;

	// If the noise generator enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth")
	bool bNoiseEnabled = false;

	// The volume of the noise source
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (EditCondition = "bNoiseEnabled"))
	FVector2D NoiseVolumeRange = { 1.0f, 1.0f };

	// If the noise generator has the envelope enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (EditCondition = "bNoiseEnabled"))
	bool bNoiseEnvelopeEnabled = false;

	// The filter frequency of the noise generator between min/max RPM
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (EditCondition = "bNoiseEnabled"))
	FVector2D NoiseLPFRange = { 500.0f, 500.0f };

	// Noise envelope attack time in milliseconds between min/max RPM
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (EditCondition = "bNoiseEnabled"))
	FVector2D NoiseAttackTimeMsecRange = { 10.0f, 10.0f };

	// Noise envelope attack curve.  1.0 = linear growth, < 1.0 logorithmic growth, > 1.0 exponential growth.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (EditCondition = "bNoiseEnabled"))
	FVector2D NoiseAttackCurveRange = { 1.0f, 1.0f };

	// Noise envelope attack time in milliseconds between min/max RPM
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (EditCondition = "bNoiseEnabled"))
	FVector2D NoiseDecayTimeMsecRange = { 10.0f, 10.0f };

	// Noise envelope attack curve between min/max RPM.  1.0 = linear growth, < 1.0 logorithmic growth, > 1.0 exponential growth.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth", meta = (EditCondition = "bNoiseEnabled"))
	FVector2D NoiseDecayCurveRange = { 1.0f, 1.0f };

	// If the granular engine is enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Granular Engine")
	bool bGranularEngineEnabled = true;

	// The volume of the granular engine
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Granular Engine", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "bGranularEngineEnabled"))
	float GranularEngineVolume = 1.0f;

	// The pitch scale of the granular engine
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Granular Engine", meta = (ClampMin = "0.001", UIMin = "0.001", EditCondition = "bGranularEngineEnabled"))
	float GranularEnginePitchScale = 1.0f;

	// The volume of the granular engine
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Granular Engine", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bGranularEngineEnabled"))
	int32 NumSamplesToCrossfadeBetweenGrains = 10;

	// How many grain-table entries to use per runtime grain
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Granular Engine", meta = (ClampMin = "1", ClampMax = "20", UIMin = "1", UIMax = "20", EditCondition = "bGranularEngineEnabled"))
	int32 NumGrainTableEntriesPerGrain = 3;

	// Random grain table offset for cases where RPM is constant. Allows random shuffling of grains to avoid a robotic sound.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Granular Engine", meta = (ClampMin = "0", ClampMax = "50", UIMin = "0", UIMax = "50", EditCondition = "bGranularEngineEnabled"))
	int32 GrainTableRandomOffsetForConstantRPMs = 20;

	// Number of samples to cross fade grains when on a constant-RPM state. More crossfaded samples can reduce the robotic sound.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Granular Engine", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bGranularEngineEnabled"))
	int32 GrainCrossfadeSamplesForConstantRPMs = 20;

	// Motosynth source to use for granular engine acceleration
	UPROPERTY(EditAnywhere, Category = "Granular Engine", meta = (EditCondition = "bGranularEngineEnabled"))
	TObjectPtr<UMotoSynthSource> AccelerationSource = nullptr;

	// Motosynth source to use for granular engine deceleration
	UPROPERTY(EditAnywhere, Category = "Granular Engine", meta = (EditCondition = "bGranularEngineEnabled"))
	TObjectPtr<UMotoSynthSource> DecelerationSource = nullptr;

	// If the stereo widener is enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects")
	bool bStereoWidenerEnabled = true;
	
	// If the stereo widener is enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects", meta = (ClampMin = "0.0", ClampMax = "200.0", UIMin = "0.0", UIMax = "200.0", EditCondition = "bStereoWidenerEnabled"))
	float StereoDelayMsec = 25.0f;

	// Amount of feedback for stereo widener
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "bStereoWidenerEnabled"))
	float StereoFeedback = 0.37f;

	// Wet level of stereo delay used for stereo widener
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "bStereoWidenerEnabled"))
	float StereoWidenerWetlevel = 0.68f;

	// Dry level of stereo delay used for stereo widener
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "bStereoWidenerEnabled"))
	float StereoWidenerDryLevel = 0.8f;

	// Delay ratio of left/right channels for stereo widener effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "bStereoWidenerEnabled"))
	float StereoWidenerDelayRatio = 0.43f;

	// Delay ratio of left/right channels for stereo widener effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects", meta = (EditCondition = "bStereoWidenerEnabled"))
	bool bStereoWidenerFilterEnabled = true;

	// Delay ratio of left/right channels for stereo widener effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects", meta = (ClampMin = "20.0", ClampMax = "20000.0", UIMin = "20.0", UIMax = "20000.0", EditCondition = "bStereoWidenerEnabled && bStereoWidenerFilterEnabled"))
	float StereoWidenerFilterFrequency = 4000.0f;

	// Delay ratio of left/right channels for stereo widener effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Effects", meta = (ClampMin = "0.0", ClampMax = "2.0", UIMin = "0.0", UIMax = "2.0", EditCondition = "bStereoWidenerEnabled && bStereoWidenerFilterEnabled"))
	float StereoWidenerFilterQ = 0.5f;
};


#if WITH_EDITOR
/**
* FMotoSynthEnginePreviewer
* Class used to render the moto synth audio in the content browser. Used to preview the moto synth engine without running PIE.
*/
class FMotoSynthEnginePreviewer : public ISubmixBufferListener
{
public:
	FMotoSynthEnginePreviewer();
	virtual ~FMotoSynthEnginePreviewer() override;

	void StartPreviewing();
	void StopPreviewing();
	void Reset();

	void SetSettings(const FMotoSynthRuntimeSettings& InSettings);
	void SetPreviewRPMCurve(const FRichCurve& InRPMCurve);

	// ISubmixBufferListener
	void OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 NumChannels, const int32 InSampleRate, double AudioClock) override;
	const FString& GetListenerName() const override;
	// ~ ISubmixBufferListener

private:
	FCriticalSection PreviewEngineCritSect;
	FRichCurve PreviewRPMCurve;

	float CurrentPreviewCurveStartTime = 0.0f;
	float CurrentPreviewCurveTime = 0.0f;

	TArray<float> OutputBuffer;
	TUniquePtr<FMotoSynthEngine> SynthEngine;

	FVector2D RPMRange;

	UMotoSynthPreset* MotoSynthPreset;
	FMotoSynthRuntimeSettings Settings;

	bool bRegistered = false;
	bool bEngineInitialized = false;
	bool bPreviewFinished = false;
};
#endif // WITH_EDITOR

/** 
* Asset used to store moto synth preset data.
*/
UCLASS(MinimalAPI, ClassGroup = Synth, meta = (BlueprintSpawnableComponent))
class UMotoSynthPreset : public UObject
{
	GENERATED_BODY()

public:

	UE_API virtual void BeginDestroy() override;
	UE_API virtual void PostLoad() override;

#if WITH_EDITORONLY_DATA
	// Engine preview RPM curve
	UPROPERTY(EditAnywhere, Category = "Engine Preview")
	FRuntimeFloatCurve EnginePreviewRPMCurve;
#endif // #if WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MotoSynthSettings, Meta = (ShowOnlyInnerProperties))
	FMotoSynthRuntimeSettings Settings;

#if WITH_EDITOR
	UFUNCTION(Category = "Engine Preview", meta = (CallInEditor = "true"))
	UE_API void StartEnginePreview();

	UFUNCTION(Category = "Engine Preview", meta = (CallInEditor = "true"))
	UE_API void StopEnginePreview();

	// Dumps memory usage of the preset (i.e. of the source assets)
	UFUNCTION(Category = "Memory", meta = (CallInEditor = "true"))
	UE_API void DumpRuntimeMemoryUsage();

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

#if WITH_EDITORONLY_DATA
	// The engine previewer which is using this preset
	TSharedRef<FMotoSynthEnginePreviewer> EnginePreviewer = MakeShared<FMotoSynthEnginePreviewer>();
#endif
};

#undef UE_API
