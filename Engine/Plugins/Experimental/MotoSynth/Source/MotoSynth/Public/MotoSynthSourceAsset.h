// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/CurveFloat.h"
#include "DSP/Osc.h"
#include "DSP/Filter.h"
#include "ISubmixBufferListener.h"
#include "MotoSynthSourceAsset.generated.h"

#define UE_API MOTOSYNTH_API

class USoundWave;
struct FPropertyChangedEvent;
class UMotoSynthSource;

USTRUCT()
struct FGrainTableEntry
{
	GENERATED_BODY()

	UPROPERTY()
	int32 SampleIndex = 0;

	// The RPM of the grain when it starts
	UPROPERTY()
	float RPM = 0.0f;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	int32 AnalysisSampleIndex = 0;
#endif
};

#if WITH_EDITOR
// Class for playing a match tone for estimating RPMs
class FRPMEstimationPreviewTone : public ISubmixBufferListener
{
public:
	FRPMEstimationPreviewTone();
	virtual ~FRPMEstimationPreviewTone();

	void StartTestTone(float InVolume);
	void StopTestTone();

	void Reset();
	void SetAudioFile(TArrayView<const int16>& InAudioFile, int32 SampleRate);
	void SetPitchCurve(FRichCurve& InRPMCurve);
	bool IsDone() const { return CurrentFrame >= AudioFileBuffer.Num(); }

	// ISubmixBufferListener
	virtual void OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate, double AudioClock) override;
	virtual const FString& GetListenerName() const override;
	// ~ISubmixBufferListener

private:
	// Critical section used to set params from game thread while synth is running
	FCriticalSection CallbackCritSect;

	// Scratch buffer to generate audio into
	TArray<float> ScratchBuffer;
	TArrayView<const int16> AudioFileBuffer;
	Audio::FOsc Osc;
	Audio::FLadderFilter Filter;
	int32 SampleRate = 0;
	FRichCurve RPMCurve;
	int32 CurrentFrame = 0;
	float VolumeScale = 1.0f;
	bool bRegistered = false;
};
#endif // WITH_EDITOR

/*
 UMotoSynthSource
 UAsset used to represent Imported MotoSynth Sources
*/
UCLASS(MinimalAPI)
class UMotoSynthSource : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMotoSynthSource();
	UE_API ~UMotoSynthSource();
	
	//~ Begin UObject interface

	// Needed to remove the data entry in the data manager
	UE_API virtual void BeginDestroy() override;

	// Used to register data w/ the data manager
	UE_API virtual void PostLoad() override;
	//~ End UObject interface

#if WITH_EDITORONLY_DATA
	// The source to use for the moto synth source
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grain Table | Analysis")
	TObjectPtr<USoundWave> SoundWaveSource;
#endif // #if WITH_EDITORONLY_DATA

	// Whether or not to convert this moto synth source to 8 bit on load to use less memory
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Memory")
	bool bConvertTo8Bit = false;

	// Amount to scale down the sample rate of the source
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Memory", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float DownSampleFactor = 1.0f;

	// A curve to define the RPM contour from the min and max estimated RPM 
	// Curve values are non-normalized and accurate to time
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grain Table | Analysis")
	FRuntimeFloatCurve RPMCurve;

#if WITH_EDITORONLY_DATA

	// Sets the volume of the RPM curve synth for testing RPM curve to source
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grain Table | Analysis")
	float RPMSynthVolume = 1.0f;

	// Whether not to enable a low pass filter frequency before analyzing the audio file
	UPROPERTY(EditAnywhere, Category = "Grain Table | Filtering")
	bool bEnableFilteringForAnalysis = true;

	// Frequency of a low pass filter to apply before running grain table analysis
	UPROPERTY(EditAnywhere, Category = "Grain Table | Filtering", meta = (ClampMin = "0.0", ClampMax = "20000.0", EditCondition = "bEnableFiltering"))
	float LowPassFilterFrequency = 500.0f;

	// Whether not to enable a low pass filter frequency before analyzing the audio file
	UPROPERTY(EditAnywhere, Category = "Grain Table | Filtering", meta = (ClampMin = "0.0", ClampMax = "20000.0", EditCondition = "bEnableFiltering"))
	float HighPassFilterFrequency = 0.0f;

	// Whether not to enable a dynamics processor to the analysis step
	UPROPERTY(EditAnywhere, Category = "Grain Table | Dynamics Processor")
	bool bEnableDynamicsProcessorForAnalysis = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grain Table | Dynamics Processor", meta = (ClampMin = "0.0", ClampMax = "50.0", UIMin = "0.0", UIMax = "50.0", EditCondition = "bEnableDynamicsProcessor"))
	float DynamicsProcessorLookahead = 1.0f;

	UPROPERTY(EditAnywhere, Category = "Grain Table | Dynamics Processor", meta = (ClampMin = "-20.0", ClampMax = "100.0", EditCondition = "bEnableDynamicsProcessor"))
	float DynamicsProcessorInputGainDb = 20.0f;

	UPROPERTY(EditAnywhere, Category = "Grain Table | Dynamics Processor", meta = (ClampMin = "1.0", ClampMax = "20.0", EditCondition = "bEnableDynamicsProcessor"))
	float DynamicsProcessorRatio = 2.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grain Table | Dynamics Processor", meta = (ClampMin = "0.0", ClampMax = "20.0", UIMin = "0.0", UIMax = "20.0", EditCondition = "bEnableDynamicsProcessor"))
	float DynamicsKneeBandwidth = 2.0f;

	UPROPERTY(EditAnywhere, Category = "Grain Table | Dynamics Processor", meta = (ClampMin = "-60.0", ClampMax = "0.0", EditCondition = "bEnableDynamicsProcessor"))
	float DynamicsProcessorThreshold = -6.0f;

	UPROPERTY(EditAnywhere, Category = "Grain Table | Dynamics Processor", meta = (ClampMin = "0.0", ClampMax = "5000.0", EditCondition = "bEnableDynamicsProcessor"))
	float DynamicsProcessorAttackTimeMsec = 1.0f;

	UPROPERTY(EditAnywhere, Category = "Grain Table | Dynamics Processor", meta = (ClampMin = "0.0", ClampMax = "5000.0", EditCondition = "bEnableDynamicsProcessor"))
	float DynamicsProcessorReleaseTimeMsec = 20.0f;

	UPROPERTY(EditAnywhere, Category = "Grain Table | Normalization")
	bool bEnableNormalizationForAnalysis = true;

	UPROPERTY(EditAnywhere, Category = "Grain Table | Grain Table Algorithm")
	int32 SampleShiftOffset = 68;

	// A samples to use to calibrate when an engine cycle begins
	UPROPERTY(EditAnywhere, Category = "Grain Table | Grain Table Algorithm")
	int32 RPMCycleCalibrationSample = 0;

	// The end of the first cycle sample. Cut the source file to start exactly on the cycle start
	UPROPERTY(EditAnywhere, Category = "Grain Table | Grain Table Algorithm")
	int32 RPMFirstCycleSampleEnd = 0;

	UPROPERTY(EditAnywhere, Category = "Grain Table | Grain Table Algorithm")
	int32 RPMEstimationOctaveOffset = 0;

	// Whether not to write the audio used for analysis to a wav file
	UPROPERTY(EditAnywhere, Category = "Grain Table | File Writing")
	bool bWriteAnalysisInputToFile = true;

	// The path to write the audio analysis data (LPF and normalized asset)
	UPROPERTY(EditAnywhere, Category = "Grain Table | File Writing", meta = (EditCondition = "bWriteAnalysisInputToFile"))
	FString AnalysisInputFilePath;
#endif

#if WITH_EDITOR

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	UFUNCTION(Category = "Grain Table", meta = (CallInEditor = "true"))
	UE_API void PerformGrainTableAnalysis();

	UFUNCTION(Category = "Grain Table", meta = (CallInEditor = "true"))
	UE_API void PlayToneMatch();

	UFUNCTION(Category = "Grain Table", meta = (CallInEditor = "true"))
	UE_API void StopToneMatch();

	// Updates the source data from the associated USoundWave
	UE_API void UpdateSourceData();

#endif // WITH_EDITOR

 	// Retrieves the data ID of the source in the moto synth data manager
 	uint32 GetDataID() const { return SourceDataID; }

	// Retrieves the memory usage of this in MB that will be used at runtime (i.e. from data manager)
	UE_API float GetRuntimeMemoryUsageMB() const;

protected:

	UE_API uint32 GetNextSourceID() const;
	UE_API void RegisterSourceData();

#if WITH_EDITOR
	UE_API float GetCurrentRPMForSampleIndex(int32 CurrentSampleIndex);
	UE_API void FilterSourceDataForAnalysis();
	UE_API void DynamicsProcessForAnalysis();
	UE_API void NormalizeForAnalysis();
	UE_API void BuildGrainTableByRPMEstimation();
	UE_API void BuildGrainTableByFFT();

	UE_API void WriteDebugDataToWaveFiles();
	UE_API void WriteAnalysisBufferToWaveFile();
	UE_API void WriteGrainTableDataToWaveFile();
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	Audio::FAlignedFloatBuffer AnalysisBuffer;
#endif

	// Data containing PCM audio of the imported source asset (filled out by the factory)
	UPROPERTY()
	TArray<float> SourceData_DEPRECATED;

	UPROPERTY()
	TArray<int16> SourceDataPCM;

	// Sample rate of the imported sound wave and the serialized data of the granulator
	UPROPERTY()
	int32 SourceSampleRate = 0;

	// Grain table containing information about how to granulate the source data buffer.
	UPROPERTY()
	TArray<FGrainTableEntry> GrainTable;

#if WITH_EDITORONLY_DATA
	TSharedPtr<FRPMEstimationPreviewTone, ESPMode::ThreadSafe> MotoSynthSineToneTest;
#endif

	// Data ID used to track the source data with the data manager
	uint32 SourceDataID = INDEX_NONE;

#if WITH_EDITOR
	friend class UMotoSynthSourceFactory; // allow factory to fill the internal buffer
	friend class FMotoSynthSourceConverter; // allow async worker to raise flags upon completion
#endif
};

#undef UE_API
