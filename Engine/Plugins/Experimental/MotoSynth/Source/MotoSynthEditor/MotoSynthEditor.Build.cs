// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MotoSynthEditor : ModuleRules
	{
		public MotoSynthEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange
			(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"UnrealEd",
					"AudioEditor",
					"MotoSynth",
					"AudioMixer",
					"ToolMenus",
					"Slate",
					"SlateCore",
					"ContentBrowser",
				}
			);

			PrivateDependencyModuleNames.AddRange
			(
				new string[]
				{
					"AssetDefinition",
					"DeveloperSettings",
				}
			);

			PrivateIncludePathModuleNames.AddRange
			(
				new string[]
				{
					"AssetTools"
				}
			);
		}
	}
}
