// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotoSynthEditorModule.h"
#include "AudioEditorModule.h"
#include "ToolMenus.h"
#include "MotoSynthSourceFactory.h"
#include "SoundWaveAssetActionExtenderMotoSynth.h"

DEFINE_LOG_CATEGORY(LogMotoSynthEditor);

IMPLEMENT_MODULE(FMotoSynthEditorModule, MotoSynthEditor)

void FMotoSynthEditorModule::StartupModule()
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	AssetTools.RegisterAssetTypeActions(MakeShared<FAssetTypeActions_MotoSynthSource>());

	// Now that we've loaded this module, we need to register our effect preset actions
	IAudioEditorModule* AudioEditorModule = &FModuleManager::LoadModuleChecked<IAudioEditorModule>("AudioEditor");
	AudioEditorModule->RegisterEffectPresetAssetActions();

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FMotoSynthEditorModule::RegisterMenus));
}

void FMotoSynthEditorModule::ShutdownModule()
{
}

void FMotoSynthEditorModule::RegisterMenus()
{
	FMotoSynthExtension::RegisterMenus();
}
