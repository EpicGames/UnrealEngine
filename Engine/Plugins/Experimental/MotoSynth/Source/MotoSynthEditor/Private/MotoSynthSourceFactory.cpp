// Copyright Epic Games, Inc. All Rights Reserved.
#include "MotoSynthSourceFactory.h"
#include "AudioAnalytics.h"
#include "MotoSynthPreset.h"
#include "MotoSynthSourceAsset.h"
#include "MotoSynthEditorModule.h"
#include "Sound/SoundWave.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotoSynthSourceFactory)

#define LOCTEXT_NAMESPACE "MotoSynthSourceFactory"

UMotoSynthPresetFactory::UMotoSynthPresetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMotoSynthPreset::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UMotoSynthPresetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UMotoSynthPreset* NewAsset = NewObject<UMotoSynthPreset >(InParent, InName, Flags);
	Audio::Analytics::RecordEvent_Usage(TEXT("MotoSynth.PresetCreated"));
	return NewAsset;
}

UClass* FAssetTypeActions_MotoSynthSource::GetSupportedClass() const
{
	return UMotoSynthSource::StaticClass();
}

UMotoSynthSourceFactory::UMotoSynthSourceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMotoSynthSource::StaticClass();

	bCreateNew = false;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UMotoSynthSourceFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (StagedSoundWave.IsValid())
	{
		USoundWave* SoundWave = StagedSoundWave.Get();
		
		if (SoundWave)
		{
			// Warn that we're ignoring non-mono source for motosynth source. Mixing channels to mono would likely destroy the source asset anyway, so we're
			// only going to use the mono channel (left) as the source
			if (SoundWave->NumChannels > 1)
			{
				UE_LOGF(LogMotoSynthEditor, Warning, "Sound source used as moto synth source has more than one channel. Only using the 0th channel index (left) for moto synth source.");
			}

			UMotoSynthSource* NewAsset = NewObject<UMotoSynthSource>(InParent, InName, Flags);
			
			NewAsset->SoundWaveSource = SoundWave;
			NewAsset->UpdateSourceData();
			NewAsset->PerformGrainTableAnalysis();

			if (FEngineAnalytics::IsAvailable())
			{
				FEngineAnalytics::GetProvider().RecordEvent(TEXT("Audio.Usage.MotoSynth.SourceCreated"));
			}

			return NewAsset;
		}

		StagedSoundWave.Reset();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
