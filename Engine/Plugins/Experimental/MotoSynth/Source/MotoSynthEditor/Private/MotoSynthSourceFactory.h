// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "Factories/Factory.h"
#include "AssetTypeActions_Base.h"
#include "MotoSynthPreset.h"
#include "MotoSynthSourceFactory.generated.h"

class USoundWave;

UCLASS()
class UAssetDefinition_MotoSynthPreset : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_MotoSynthPreset", "Moto Synth Preset"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMotoSynthPreset::StaticClass(); }
	virtual FLinearColor GetAssetColor() const override { return FColor(0, 150, 200); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio, NSLOCTEXT("AssetDefinition", "AssetDefinition_MotoSynthPresetSubMenu", "Legacy"))
		};
		return Categories;
	}
	// UAssetDefinition End
};


UCLASS(MinimalAPI, hidecategories = Object)
class UMotoSynthPresetFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface
};

class FAssetTypeActions_MotoSynthSource : public FAssetTypeActions_Base
{
public:
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MotoSynthSource", "Moto Synth Source"); }
	virtual FColor GetTypeColor() const override { return FColor(0, 255, 255); }
	virtual UClass* GetSupportedClass() const override;
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Sounds; }
};

UCLASS(MinimalAPI, hidecategories = Object)
class UMotoSynthSourceFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface

	TWeakObjectPtr<USoundWave> StagedSoundWave;
};
