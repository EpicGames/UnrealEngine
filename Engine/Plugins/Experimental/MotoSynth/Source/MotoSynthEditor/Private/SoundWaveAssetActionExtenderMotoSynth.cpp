// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundWaveAssetActionExtenderMotoSynth.h"

#include "AssetDefinitionRegistry.h"
#include "AssetTypeActions/AssetDefinition_SoundBase.h"
#include "Misc/PackageName.h"
#include "ToolMenus.h"
#include "Logging/MessageLog.h"
#include "ContentBrowserMenuContexts.h"
#include "Sound/SoundWaveProcedural.h"
#include "MotoSynthSourceFactory.h"
#include "MotoSynthSourceAsset.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

void FMotoSynthExtension::RegisterMenus()
{
	FToolMenuOwnerScoped MenuOwner("MotoSynth");

	const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(USoundWave::StaticClass());
	FToolMenuSection* Section = CastChecked<UAssetDefinition_SoundAssetBase>(AssetDefinition)->FindSoundContextMenuSection("Sound");
	check(Section);

	Section->AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
		if (!Context || Context->SelectedAssets.IsEmpty())
		{
			return;
		}

		for (const FAssetData& Object : Context->SelectedAssets)
		{
			if (Object.IsInstanceOf(USoundWaveProcedural::StaticClass()))
			{
				return;
			}
		}

		const TAttribute<FText> Label = LOCTEXT("SoundWave_CreateMotoSource", "Create MotoSynth Source");
		const TAttribute<FText> ToolTip = LOCTEXT("SoundWave_CreateMotoSynthSourceTooltip", "Creates a MotoSynth Source asset using the selected sound wave.");
		const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.MotoSynthSource");
		const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&FMotoSynthExtension::ExecuteCreateMotoSynthSource);

		InSection.AddMenuEntry("SoundWave_CreateMotoSynthSource", Label, ToolTip, Icon, UIAction);
	}));
}

void FMotoSynthExtension::ExecuteCreateMotoSynthSource(const FToolMenuContext& MenuContext)
{
	UContentBrowserAssetContextMenuContext* Context = MenuContext.FindContext<UContentBrowserAssetContextMenuContext>();
	if (!Context || Context->SelectedAssets.Num() == 0)
	{
		return;
	}

	const FString DefaultSuffix = TEXT("_MotoSynthSource");
	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");

	// Create the factory used to generate the asset
	UMotoSynthSourceFactory* Factory = NewObject<UMotoSynthSourceFactory>();
	
	FMessageLog EditorErrors("EditorErrors");
	for (const FAssetData& Asset : Context->SelectedAssets)
	{
		// stage the soundwave on the factory to be used during asset creation
		if (USoundWave* Wave = Cast<USoundWave>(Asset.GetAsset()))
		{
			Factory->StagedSoundWave = Wave; // WeakPtr gets reset by the Factory after it is consumed

			// Determine an appropriate name
			FString Name;
			FString PackagePath;
			AssetToolsModule.Get().CreateUniqueAssetName(Wave->GetOutermost()->GetName(), DefaultSuffix, PackagePath, Name);

			// create new asset
			AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackagePath), UMotoSynthSource::StaticClass(), Factory);
		}
		else
		{
			EditorErrors.Error(LOCTEXT("ExpectedUSoundWave", "Expected SoundWave"))->AddToken(FAssetNameToken::Create(Asset.PackageName.ToString()));
		}
	}
	EditorErrors.Notify();
}

#undef LOCTEXT_NAMESPACE
