// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FMotoSynthExtension
{
public:
	static void RegisterMenus();
	static void ExecuteCreateMotoSynthSource(const struct FToolMenuContext& MenuContext);
};

